// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class PlacementMode : ModuleRules
{
    public PlacementMode(ReadOnlyTargetRules Target) : base(Target)
    {
		PrivateIncludePathModuleNames.Add("AssetTools");

        PublicDependencyModuleNames.AddRange( 
            new string[] { 
                "Engine", 
            } 
        );
        
        PrivateDependencyModuleNames.AddRange( 
            new string[] { 
                "Core", 
                "CoreUObject",
                "InputCore",
                "Slate",
				"SlateCore",
				"EditorFramework",
				"UnrealEd",
                "ContentBrowser",
				"ContentBrowserData",
                "CollectionManager",
                "LevelEditor",
                "AssetTools",
                "EditorWidgets",
                "ToolMenus",
				"TypedElementFramework",
				"TypedElementRuntime",
				"WidgetRegistration"
            } 
        );
    }
}
