// Copyright Epic Games, Inc. All Rights Reserved.
#include "PlacementModeModule.h"

#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "UObject/Object.h"
#include "Misc/Guid.h"
#include "Misc/NamePermissionList.h"
#include "UObject/Class.h"
#include "UObject/UObjectHash.h"
#include "UObject/UObjectIterator.h"
#include "Textures/SlateIcon.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "GameFramework/Actor.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorFactories/ActorFactoryBoxReflectionCapture.h"
#include "ActorFactories/ActorFactoryBoxVolume.h"
#include "ActorFactories/ActorFactoryCharacter.h"
#include "ActorFactories/ActorFactoryDeferredDecal.h"
#include "ActorFactories/ActorFactoryDirectionalLight.h"
#include "ActorFactories/ActorFactoryEmptyActor.h"
#include "ActorFactories/ActorFactoryPawn.h"
#include "ActorFactories/ActorFactoryExponentialHeightFog.h"
#include "ActorFactories/ActorFactoryLocalFogVolume.h"
#include "ActorFactories/ActorFactorySkyAtmosphere.h"
#include "ActorFactories/ActorFactoryVolumetricCloud.h"
#include "ActorFactories/ActorFactoryPlayerStart.h"
#include "ActorFactories/ActorFactoryPointLight.h"
#include "ActorFactories/ActorFactorySpotLight.h"
#include "ActorFactories/ActorFactoryRectLight.h"
#include "ActorFactories/ActorFactorySkyLight.h"
#include "ActorFactories/ActorFactorySphereReflectionCapture.h"
#include "ActorFactories/ActorFactoryBasicShape.h"
#include "ActorFactories/ActorFactoryTriggerBox.h"
#include "ActorFactories/ActorFactoryTriggerSphere.h"
#include "Engine/BrushBuilder.h"
#include "Engine/Brush.h"
#include "Engine/StaticMesh.h"
#include "GameFramework/Volume.h"
#include "Engine/PostProcessVolume.h"
#include "LevelEditorActions.h"
#include "AssetRegistry/AssetData.h"
#include "EditorModeRegistry.h"
#include "EditorModes.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ActorPlacementInfo.h"
#include "IPlacementModeModule.h"
#include "Subsystems/PlacementSubsystem.h"
#include "ToolMenus.h"
#include "AssetToolsModule.h"
#include "Kismet2/KismetEditorUtilities.h"
#include "ActorFactories/ActorFactoryPlanarReflection.h"
#include "SPlacementModeTools.h"
#include "AssetSelection.h"
#include "PlacementModeLog.h"

#define LOCTEXT_NAMESPACE "PlacementMode"

IMPLEMENT_MODULE(FPlacementModeModule, PlacementMode);

namespace PlacementModeModuleLocals
{
	FActorPlacementInfo MakePlacementInfo(const FPlaceableItem& Item)
	{
		FString ObjectString = Item.AssetData.IsValid() ? Item.AssetData.GetObjectPathString() : FString();
		FString FactoryString = Item.AssetFactory ? Item.AssetFactory.GetObject()->GetPathName() : FString();
		return FActorPlacementInfo(ObjectString, FactoryString);
	}
}

TOptional<FLinearColor> GetBasicShapeColorOverride()
{
	// Get color for basic shapes.  It should appear like all the other basic types
	static TOptional<FLinearColor> BasicShapeColorOverride;

	if (!BasicShapeColorOverride.IsSet())
	{
		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));
		TSharedPtr<IAssetTypeActions> AssetTypeActions;
		AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(UClass::StaticClass()).Pin();
		if (AssetTypeActions.IsValid())
		{
			BasicShapeColorOverride = TOptional<FLinearColor>(AssetTypeActions->GetTypeColor());
		}
	}
	return BasicShapeColorOverride;
}

FPlacementModeModule::FPlacementModeModule()
	: CategoryPermissionList(MakeShared<FNamePermissionList>())
{
	CategoryPermissionList->OnFilterChanged().AddRaw(this, &FPlacementModeModule::OnCategoryPermissionListChanged);
}

void FPlacementModeModule::StartupModule()
{
	using namespace PlacementModeModuleLocals;

	TArray< FString > RecentlyPlacedAsStrings;
	GConfig->GetArray(TEXT("PlacementMode"), TEXT("RecentlyPlaced"), RecentlyPlacedAsStrings, GEditorPerProjectIni);

	for (int Index = 0; Index < RecentlyPlacedAsStrings.Num(); Index++)
	{
		RecentlyPlaced.Add(FActorPlacementInfo(RecentlyPlacedAsStrings[Index]));
	}

	if (!IsRunningCommandlet())
	{
		IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
		AssetRegistry.OnAssetRemoved().AddRaw(this, &FPlacementModeModule::OnAssetRemoved);
		AssetRegistry.OnAssetRenamed().AddRaw(this, &FPlacementModeModule::OnAssetRenamed);
		if (AssetRegistry.IsLoadingAssets())
		{
			AssetRegistry.OnFilesLoaded().AddRaw(this, &FPlacementModeModule::OnInitialAssetsScanComplete);
		}
		else
		{
			AssetRegistry.OnAssetAdded().AddRaw(this, &FPlacementModeModule::OnAssetAdded);
		}
	}

	TOptional<FLinearColor> BasicShapeColorOverride = GetBasicShapeColorOverride();


	RegisterPlacementCategory(
		FPlacementCategoryInfo(
			LOCTEXT("Favorites", "Favorites"),
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), "Icons.Favorites.Small"),
			FBuiltInPlacementCategories::Favorites(),
			TEXT("Favorites"),
			TNumericLimits<int32>::Lowest(),
			false
		));
	
	RegisterPlacementCategory(
		FPlacementCategoryInfo(
			LOCTEXT("RecentlyPlaced", "Recent"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.Recent"),
			FBuiltInPlacementCategories::RecentlyPlaced(),
			TEXT("PMRecentlyPlaced"),
			TNumericLimits<int32>::Lowest() + 1,
			false
		)
	);

	{
		int32 SortOrder = 0;
		FName CategoryName = FBuiltInPlacementCategories::Basic();
		RegisterPlacementCategory(
			FPlacementCategoryInfo(
				LOCTEXT("Basic", "Basic"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.Basic"),
				CategoryName,
				TEXT("PMBasic"),
				10
			)
		);

		FPlacementCategory* Category = Categories.Find(CategoryName);
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryEmptyActor::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryCharacter::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryPawn::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryPointLight::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryPlayerStart::StaticClass(), SortOrder += 10));
	
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryTriggerBox::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryTriggerSphere::StaticClass(), SortOrder += 10));
	}

	{
		int32 SortOrder = 0;
		FName CategoryName = FBuiltInPlacementCategories::Lights();
		RegisterPlacementCategory(
			FPlacementCategoryInfo(
				LOCTEXT("Lights", "Lights"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.Lights"),
				CategoryName,
				TEXT("PMLights"),
				20
			)
		);

		FPlacementCategory* Category = Categories.Find(CategoryName);
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryDirectionalLight::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryPointLight::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactorySpotLight::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryRectLight::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactorySkyLight::StaticClass(), SortOrder += 10));
	}

	{

		int32 SortOrder = 0;
		FName CategoryName = FBuiltInPlacementCategories::Shapes();
		RegisterPlacementCategory(
			FPlacementCategoryInfo(
				LOCTEXT("Shapes", "Shapes"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.Shapes"),
				CategoryName,
				TEXT("PMShapes"),
				25	
				)
			);

		static const FText CubeText = LOCTEXT("Cube", "Cube");
		static const FText SphereText = LOCTEXT("Sphere", "Sphere");
		static const FText CylinderText = LOCTEXT("Cylinder", "Cylinder");
		static const FText ConeText = LOCTEXT("Cone", "Cone");
		static const FText PlaneText = LOCTEXT("Plane", "Plane");

		FPlacementCategory* Category = Categories.Find(CategoryName);
		// Cube
		{
			TSharedPtr<FPlaceableItem> Cube = MakeShared<FPlaceableItem>(*UActorFactoryBasicShape::StaticClass(), FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicCube.ToString())), FName("ClassThumbnail.Cube"), FName("ClassIcon.Cube"), BasicShapeColorOverride, SortOrder += 10, CubeText);
			ManuallyCreatedPlaceableItems.Add(MakePlacementInfo(*Cube), Cube);
			Category->Items.Add(CreateID(), Cube);
		}
		// Sphere
		{
			TSharedPtr<FPlaceableItem> Sphere = MakeShared<FPlaceableItem>(*UActorFactoryBasicShape::StaticClass(), FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicSphere.ToString())), FName("ClassThumbnail.Sphere"), FName("ClassIcon.Sphere"), BasicShapeColorOverride, SortOrder += 10, SphereText);
			ManuallyCreatedPlaceableItems.Add(MakePlacementInfo(*Sphere), Sphere);
			Category->Items.Add(CreateID(), Sphere);
		}
		// Cylinder
		{
			TSharedPtr<FPlaceableItem> Cylinder = MakeShared<FPlaceableItem>(*UActorFactoryBasicShape::StaticClass(), FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicCylinder.ToString())), FName("ClassThumbnail.Cylinder"), FName("ClassIcon.Cylinder"), BasicShapeColorOverride, SortOrder += 10, CylinderText);
			ManuallyCreatedPlaceableItems.Add(MakePlacementInfo(*Cylinder), Cylinder);
			Category->Items.Add(CreateID(), Cylinder);
		}
		// Cone
		{
			TSharedPtr<FPlaceableItem> Cone = MakeShared<FPlaceableItem>(*UActorFactoryBasicShape::StaticClass(), FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicCone.ToString())), FName("ClassThumbnail.Cone"), FName("ClassIcon.Cone"), BasicShapeColorOverride, SortOrder += 10, ConeText);
			ManuallyCreatedPlaceableItems.Add(MakePlacementInfo(*Cone), Cone);
			Category->Items.Add(CreateID(), Cone);
		}
		// Plane
		{
			TSharedPtr<FPlaceableItem> Plane = MakeShared<FPlaceableItem>(*UActorFactoryBasicShape::StaticClass(), FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicPlane.ToString())), FName("ClassThumbnail.Plane"), FName("ClassIcon.Plane"), BasicShapeColorOverride, SortOrder += 10, PlaneText);
			ManuallyCreatedPlaceableItems.Add(MakePlacementInfo(*Plane), Plane);
			Category->Items.Add(CreateID(), Plane);
		}
	}

	{
		int32 SortOrder = 0;
		FName CategoryName = FBuiltInPlacementCategories::Visual();

		FPlacementCategoryInfo VfxCategoryInfo = FPlacementCategoryInfo(
				LOCTEXT("VisualEffects", "Visual Effects"),
				FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.VisualEffects"),
				CategoryName,
				TEXT("PMVisual"),
				30
			);
		VfxCategoryInfo.ShortDisplayName = LOCTEXT("VisualEffectsShortCategoryName", "VFX");
		RegisterPlacementCategory(MoveTemp(VfxCategoryInfo));

		UActorFactory* PPFactory = GEditor->FindActorFactoryByClassForActorClass(UActorFactoryBoxVolume::StaticClass(), APostProcessVolume::StaticClass());

		FPlacementCategory* Category = Categories.Find(CategoryName);
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(PPFactory, FAssetData(APostProcessVolume::StaticClass()), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactorySkyAtmosphere::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryVolumetricCloud::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryExponentialHeightFog::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryLocalFogVolume::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactorySphereReflectionCapture::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryBoxReflectionCapture::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryPlanarReflection::StaticClass(), SortOrder += 10));
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryDeferredDecal::StaticClass(), SortOrder += 10));
	}

	RegisterPlacementCategory(
		FPlacementCategoryInfo(
			LOCTEXT("Volumes", "Volumes"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.Volumes"),
			FBuiltInPlacementCategories::Volumes(),
			TEXT("PMVolumes"),
			40
		)
	);

	RegisterPlacementCategory(
		FPlacementCategoryInfo(
			LOCTEXT("AllClasses", "All"),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "PlacementBrowser.Icons.All"),
			FBuiltInPlacementCategories::AllClasses(),
			TEXT("PMAllClasses"),
			50
		)
	);

	if (FSlateApplication::IsInitialized())
	{
		// Given a Category Name, this will add a section and all the placeable items in that section directly into the menu
		auto GenerateQuickCreateSection = [this](const FName& SectionName, UToolMenu* InMenu, int MaxItems = 20)
		{
			FPlacementCategory* Category = Categories.Find(SectionName);
			FToolMenuSection& Section = InMenu->AddSection(SectionName, Category->DisplayName);

			int count = 1;
			for (auto& Pair : Category->Items)
			{
				TSharedPtr<const FPlaceableItem> Item = Pair.Value;

				// Use NativeName instead of AssetName to allow for multiple item to be registered otherwise can't have different item sharing same FAssetData
				FToolMenuEntry& NewEntry = Section.AddEntry(
					FToolMenuEntry::InitWidget(FName(Item->NativeName), SNew(SPlacementAssetMenuEntry, Item), FText(), true, true)
				);

				if (++count > MaxItems)
					break;
			}
		};

		auto GenerateCategorySubMenu = [this](UToolMenu* InMenu, const FName& InSectionName, const FText& InSectionDisplayName, const FName& PlacementCategory)
		{
			
			RegenerateItemsForCategory(PlacementCategory);
			FPlacementCategory* Category = Categories.Find(PlacementCategory);
			if (!Category->Items.IsEmpty())
			{
				FToolMenuSection* InSection = InMenu->FindSection(InSectionName);	
				if (InSection == nullptr)
				{
					InSection = &(InMenu->AddSection(InSectionName, InSectionDisplayName));
				}

				FToolMenuEntry& AllSubMenu = InSection->AddSubMenu(PlacementCategory,
					Category->DisplayName,
					FText::GetEmpty(),
					FNewToolMenuDelegate::CreateLambda([this, PlacementCategory](UToolMenu* InMenu)
					{
						FToolMenuSection& Section = InMenu->AddSection(PlacementCategory);
						RegenerateItemsForCategory(PlacementCategory);
						FPlacementCategory* Category = Categories.Find(PlacementCategory);
						for (auto& Pair : Category->Items)
						{
							TSharedPtr<const FPlaceableItem> Item = Pair.Value;
							// Use NativeName instead of AssetName to allow for multiple item to be registered otherwise can't have different item sharing same FAssetData
							FToolMenuEntry& NewEntry = Section.AddEntry(
								FToolMenuEntry::InitWidget(FName(Item->NativeName), SNew(SPlacementAssetMenuEntry, Item), FText(), true, true)
							);
						}
					})
				);
				AllSubMenu.Icon = Category->DisplayIcon;
			}
		};

		
		UToolMenu* ContentMenu = UToolMenus::Get()->ExtendMenu("LevelEditor.LevelEditorToolBar.AddQuickMenu");
		FName CreateSectionName = TEXT("PMQCreateMenu");
		FText CreateSectionDisplayName = LOCTEXT("PMQCreateMenu", "Place Actors");

		GenerateCategorySubMenu(ContentMenu, CreateSectionName, CreateSectionDisplayName, FBuiltInPlacementCategories::Basic());

		// All Subcategories as submenus
		FName CategoriesSectionName = TEXT("CreateAllCategories");
		ContentMenu->AddDynamicSection(CategoriesSectionName,
			FNewToolMenuDelegate::CreateLambda([this, CreateSectionName, CreateSectionDisplayName, GenerateCategorySubMenu](UToolMenu* InMenu)
			{
				TArray<FPlacementCategoryInfo> SortedCategories;
				GetSortedCategories(SortedCategories);
				for (auto CategoryInfo : SortedCategories)
				{
					// Skip Basic and Recent since we add those later
					if (CategoryInfo.UniqueHandle == FBuiltInPlacementCategories::Basic() ||
						CategoryInfo.UniqueHandle == FBuiltInPlacementCategories::RecentlyPlaced())
						continue;

					GenerateCategorySubMenu(InMenu, CreateSectionName, CreateSectionDisplayName, CategoryInfo.UniqueHandle);
				}
			}
		), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));


		// Recents Section, limit to 5 items
		const FName& RecentName = FBuiltInPlacementCategories::RecentlyPlaced();

		ContentMenu->AddDynamicSection(RecentName,
			FNewToolMenuDelegate::CreateLambda([this, GenerateQuickCreateSection, GenerateCategorySubMenu, CreateSectionName, CreateSectionDisplayName](UToolMenu* InMenu)
			{
				const FName& RecentName = FBuiltInPlacementCategories::RecentlyPlaced();
				FPlacementCategory* RecentCategory = Categories.Find(RecentName);
				RefreshRecentlyPlaced();
				GenerateQuickCreateSection(RecentName, InMenu, 5);
			}
		));

		// Open Placement Browser Panel
		FToolMenuSection& BrowserSection = ContentMenu->AddSection(TEXT("PlacementBrowserMenuSection"), FText::GetEmpty(), FToolMenuInsert(RecentName, EToolMenuInsertType::Before));

		BrowserSection.AddMenuEntry(FLevelEditorCommands::Get().OpenPlaceActors);

	} // end FSlateApplication::IsInitialized()

}

void FPlacementModeModule::PreUnloadCallback()
{
	FAssetRegistryModule* AssetRegistryModule = FModuleManager::GetModulePtr<FAssetRegistryModule>("AssetRegistry");
	if (AssetRegistryModule)
	{
		IAssetRegistry* AssetRegistry = AssetRegistryModule->TryGet();
		if (AssetRegistry)
		{
			AssetRegistry->OnAssetRemoved().RemoveAll(this);
			AssetRegistry->OnAssetRenamed().RemoveAll(this);
			AssetRegistry->OnAssetAdded().RemoveAll(this);
			AssetRegistry->OnFilesLoaded().RemoveAll(this);
		}
	}
}

void FPlacementModeModule::AddToRecentlyPlaced(const TArray<UObject*>& PlacedObjects, UActorFactory* FactoryUsed /* = NULL */)
{
	AddToRecentlyPlaced(PlacedObjects, TScriptInterface<IAssetFactoryInterface>(FactoryUsed));
}

void FPlacementModeModule::AddToRecentlyPlaced(const TArray< UObject* >& PlacedObjects, TScriptInterface<IAssetFactoryInterface> FactoryUsed)
{
	FString FactoryPath;
	if (FactoryUsed != NULL)
	{
		FactoryPath = FactoryUsed.GetObject()->GetPathName();
	}

	TArray< UObject* > FilteredPlacedObjects;
	for (UObject* PlacedObject : PlacedObjects)
	{
		// Don't include null placed objects that just have factories.
		if (PlacedObject == NULL)
		{
			continue;
		}

		// Don't add brush builders to the recently placed.
		if (PlacedObject->IsA(UBrushBuilder::StaticClass()))
		{
			continue;
		}

		FilteredPlacedObjects.Add(PlacedObject);
	}

	// Don't change the recently placed if nothing passed the filter.
	if (FilteredPlacedObjects.Num() == 0)
	{
		return;
	}

	bool Changed = false;
	for (int Index = 0; Index < FilteredPlacedObjects.Num(); Index++)
	{
		Changed |= RecentlyPlaced.Remove(FActorPlacementInfo(FilteredPlacedObjects[Index]->GetPathName(), FactoryPath)) > 0;
	}

	for (int Index = 0; Index < FilteredPlacedObjects.Num(); Index++)
	{
		if (FilteredPlacedObjects[Index] != NULL)
		{
			RecentlyPlaced.Insert(FActorPlacementInfo(FilteredPlacedObjects[Index]->GetPathName(), FactoryPath), 0);
			Changed = true;
		}
	}

	for (int Index = RecentlyPlaced.Num() - 1; Index >= 20; Index--)
	{
		RecentlyPlaced.RemoveAt(Index);
		Changed = true;
	}

	if (Changed)
	{
		TArray< FString > RecentlyPlacedAsStrings;
		for (int Index = 0; Index < RecentlyPlaced.Num(); Index++)
		{
			RecentlyPlacedAsStrings.Add(RecentlyPlaced[Index].ToString());
		}

		GConfig->SetArray(TEXT("PlacementMode"), TEXT("RecentlyPlaced"), RecentlyPlacedAsStrings, GEditorPerProjectIni);
		RecentlyPlacedChanged.Broadcast(RecentlyPlaced);
	}
}

void FPlacementModeModule::OnAssetRemoved(const FAssetData&)
{
	RecentlyPlacedChanged.Broadcast(RecentlyPlaced);
	AllPlaceableAssetsChanged.Broadcast();
}

void FPlacementModeModule::OnAssetRenamed(const FAssetData& AssetData, const FString& OldObjectPath)
{
	for (auto& RecentlyPlacedItem : RecentlyPlaced)
	{
		if (RecentlyPlacedItem.ObjectPath == OldObjectPath)
		{
			RecentlyPlacedItem.ObjectPath = AssetData.GetObjectPathString();
			break;
		}
	}

	RecentlyPlacedChanged.Broadcast(RecentlyPlaced);
	AllPlaceableAssetsChanged.Broadcast();
}

void FPlacementModeModule::OnAssetAdded(const FAssetData& AssetData)
{
	AllPlaceableAssetsChanged.Broadcast();
}

void FPlacementModeModule::OnInitialAssetsScanComplete()
{
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry")).Get();
	AssetRegistry.OnAssetAdded().AddRaw(this, &FPlacementModeModule::OnAssetAdded);
	AssetRegistry.OnFilesLoaded().RemoveAll(this);
	
	RegenerateItemsForCategory(FBuiltInPlacementCategories::AllClasses());
	UE_LOGF(LogPlacementMode, Display, "The Asset Registry is done with its initial scan, the list of placeable classes has been updated.")
	AllPlaceableAssetsChanged.Broadcast();
}

void FPlacementModeModule::AddToRecentlyPlaced(UObject* Asset, UActorFactory* FactoryUsed /* = NULL */)
{
	AddToRecentlyPlaced(Asset, TScriptInterface<IAssetFactoryInterface>(FactoryUsed));
}

void FPlacementModeModule::AddToRecentlyPlaced(UObject* Asset, TScriptInterface<IAssetFactoryInterface> FactoryUsed)
{
	TArray< UObject* > Assets;
	Assets.Add(Asset);
	AddToRecentlyPlaced(Assets, FactoryUsed);
}

TSharedRef<SWidget> FPlacementModeModule::CreatePlacementModeBrowser(TSharedRef<SDockTab> ParentTab)
{
	return SNew(SPlacementModeTools, ParentTab);
}

bool FPlacementModeModule::RegisterPlacementCategory(const FPlacementCategoryInfo& Info)
{
	if (Categories.Contains(Info.UniqueHandle))
	{
		return false;
	}

	Categories.Add(Info.UniqueHandle, Info);
	PlacementModeCategoryListChanged.Broadcast();
	return true;
}

void FPlacementModeModule::UnregisterPlacementCategory(FName Handle)
{
	if (Categories.Remove(Handle) && !IsEngineExitRequested())
	{
		PlacementModeCategoryListChanged.Broadcast();
	}
}

void FPlacementModeModule::GetSortedCategories(TArray<FPlacementCategoryInfo>& OutCategories) const
{
	TArray<FName> SortedNames;
	Categories.GenerateKeyArray(SortedNames);

	SortedNames.Sort([&](const FName& A, const FName& B) {
		return Categories[A].SortOrder < Categories[B].SortOrder;
	});

	OutCategories.Reset(Categories.Num());
	for (const FName& Name : SortedNames)
	{
		if (CategoryPermissionList->PassesFilter(Name))
		{
			OutCategories.Add(Categories[Name]);
		}
	}
}

TOptional<FPlacementModeID> FPlacementModeModule::RegisterPlaceableItem(FName CategoryName, const TSharedRef<FPlaceableItem>& InItem)
{
	using namespace PlacementModeModuleLocals;

	FPlacementCategory* Category = Categories.Find(CategoryName);
	if (Category )
	{
		if ( InItem->DragHandler.IsValid() )
		{
			Category->CustomDraggableItems.Add( InItem );
		}
		if (!Category->CustomGenerator)
		{
			FPlacementModeID ID = CreateID(CategoryName);
			Category->Items.Add(ID.UniqueID, InItem);

			ManuallyCreatedPlaceableItems.Add(MakePlacementInfo(*InItem), InItem);

			return ID;
		}
	}
	return TOptional<FPlacementModeID>();
}

void FPlacementModeModule::UnregisterPlaceableItem(FPlacementModeID ID)
{
	using namespace PlacementModeModuleLocals;

	FPlacementCategory* Category = Categories.Find(ID.Category);
	if (Category)
	{
		TSharedPtr<FPlaceableItem> Item;
		Category->Items.RemoveAndCopyValue(ID.UniqueID, Item);
		if (Item)
		{
			Category->CustomDraggableItems.Remove(Item.ToSharedRef());
			ManuallyCreatedPlaceableItems.Remove(MakePlacementInfo(*Item));
		}
	}
}

bool FPlacementModeModule::RegisterPlaceableItemFilter(TPlaceableItemPredicate Predicate, FName OwnerName)
{
	TPlaceableItemPredicate& ExistingPredicate = PlaceableItemPredicates.FindOrAdd(OwnerName);
	if (!ExistingPredicate)
	{
		ExistingPredicate = MoveTemp(Predicate);
		PlaceableItemFilteringChanged.Broadcast();
		return true;
	}
	return false;
}

void FPlacementModeModule::UnregisterPlaceableItemFilter(FName OwnerName)
{
	if (PlaceableItemPredicates.Remove(OwnerName))
	{
		PlaceableItemFilteringChanged.Broadcast();
	}
}

void FPlacementModeModule::GetItemsForCategory(FName CategoryName, TArray<TSharedPtr<FPlaceableItem>>& OutItems) const
{
	const FPlacementCategory* Category = Categories.Find(CategoryName);
	if (Category)
	{
		for (auto& Pair : Category->Items)
		{
			if (PassesFilters(Pair.Value))
			{
				OutItems.Add(Pair.Value);
			}
		}
	}
}

void FPlacementModeModule::GetFilteredItemsForCategory(FName CategoryName, TArray<TSharedPtr<FPlaceableItem>>& OutItems, TFunctionRef<bool(const TSharedPtr<FPlaceableItem> &)> Filter) const
{
	const FPlacementCategory* Category = Categories.Find(CategoryName);
	if (Category)
	{
		for (auto& Pair : Category->Items)
		{
			if (PassesFilters(Pair.Value))
			{
				if (Filter(Pair.Value))
				{
					OutItems.Add(Pair.Value);
				}
			}
		}
	}
}

void FPlacementModeModule::GetItemsWithNamesForCategory( FName CategoryName, TArray<TSharedPtr<FPlaceableItem>>& OutItems, const TArray<FName>& ItemNames ) const
{
	const FPlacementCategory* Category = Categories.Find(CategoryName);
	if (Category)
	{
		for (const TTuple<FGuid, TSharedPtr<FPlaceableItem>>& Pair : Category->Items)
		{
			TSharedPtr<FPlaceableItem> Item = Pair.Value;
			FName Name = FName( Item->NativeName );

			if ( ItemNames.Contains( Name ) )
			{
				OutItems.Add(Pair.Value);
			}
		}
	}
}

void FPlacementModeModule::RegenerateItemsForCategory(FName Category)
{
	if (Category == FBuiltInPlacementCategories::RecentlyPlaced())
	{
		RefreshRecentlyPlaced();
	}
	else if (Category == FBuiltInPlacementCategories::Volumes())
	{
		RefreshVolumes();
	}
	else if (Category == FBuiltInPlacementCategories::AllClasses() || Category == FBuiltInPlacementCategories::Favorites())
	{
		RefreshAllPlaceableClasses();
	}

	PlacementModeCategoryRefreshed.Broadcast(Category);
}

void FPlacementModeModule::RefreshRecentlyPlaced()
{
	FPlacementCategory* Category = Categories.Find(FBuiltInPlacementCategories::RecentlyPlaced());
	if (!Category)
	{
		return;
	}

	Category->Items.Reset();


	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

	for (const FActorPlacementInfo& RecentlyPlacedItem : RecentlyPlaced)
	{
		// First check if it's a manually created entry
		TWeakPtr<FPlaceableItem>* ManualItem = ManuallyCreatedPlaceableItems.Find(RecentlyPlacedItem);
		if (ManualItem)
		{
			if (ManualItem->IsValid())
			{
				Category->Items.Add(CreateID(), ManualItem->Pin());
				continue;
			}

			// A dead pointer here could theoretically be the result of an unregistered category that had 
			// items manually registered but not manually unregistered. Just remove the dead pointer and
			// try the other approaches.
			ManuallyCreatedPlaceableItems.Remove(RecentlyPlacedItem);
		}

		UObject* Asset = FindObject<UObject>(nullptr, *RecentlyPlacedItem.ObjectPath);

		// If asset is pending delete, it will not be marked as RF_Standalone, in which case we skip it
		if (Asset == nullptr || !Asset->HasAnyFlags(RF_Standalone))
		{
			continue;
		}

		FAssetData AssetData = AssetRegistryModule.Get().GetAssetByObjectPath(FSoftObjectPath(RecentlyPlacedItem.ObjectPath));

		if (!AssetData.IsValid())
		{
			continue;
		}

		TScriptInterface<IAssetFactoryInterface> Factory;
		if (RecentlyPlacedItem.Factory.IsEmpty())
		{
			// The factory portion of the recently placed data is frequently null because it only gets set
			// high up in the drop if we gave a specific factory (see FLevelEditorViewportClient::DropObjectsAtCoordinates).
			// Otherwise, the actually used factory is gotten from the placement subsystem later based on the asset data.
			// So, we do the same kind of lookup here if we don't have a factory.
			if (UPlacementSubsystem* PlacementSubsystem = GEditor->GetEditorSubsystem<UPlacementSubsystem>())
			{
				Factory = PlacementSubsystem->FindAssetFactoryFromAssetData(AssetData);
			}
		}
		else
		{
			Factory = FindObject<UObject>(nullptr, *RecentlyPlacedItem.Factory);
		}

		if (!Factory)
		{
			continue;
		}

		TSharedPtr<FPlaceableItem> Ptr = MakeShared<FPlaceableItem>(Factory, AssetData);
		Category->Items.Add(CreateID(), Ptr);
	}
}

void FPlacementModeModule::RefreshVolumes()
{
	FPlacementCategory* Category = Categories.Find(FBuiltInPlacementCategories::Volumes());
	if (!Category)
	{
		return;
	}

	Category->Items.Reset();

	// Add loaded classes
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		const UClass* Class = *ClassIt;

		if (!Class->HasAllClassFlags(CLASS_NotPlaceable) &&
			!Class->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists) &&
			Class->IsChildOf(AVolume::StaticClass()) &&
			Class->ClassGeneratedBy == nullptr)
		{
			UActorFactory* Factory = GEditor->FindActorFactoryByClassForActorClass(UActorFactoryBoxVolume::StaticClass(), Class);
			Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(Factory, FAssetData(Class)));
		}
	}
}

void FPlacementModeModule::RefreshAllPlaceableClasses()
{
	// Unregister old stuff
	FPlacementCategory* Category = Categories.Find(FBuiltInPlacementCategories::AllClasses());
	if (!Category)
	{
		return;
	}

	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FPlacementModeModule::RefreshAllPlaceableClasses");
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	if (AssetRegistry.IsLoadingAssets())
	{
		UE_LOGF(LogPlacementMode, Display, "The Asset Registry is not yet fully loaded so some placeable classes might be missing.")
	}
	
	Category->Items.Reset();


	for ( TTuple<FName, FPlacementCategory>& Pair : Categories )
	{
		for ( const TSharedRef<FPlaceableItem>& Draggable : Pair.Value.CustomDraggableItems  )
		{
			Category->Items.Add( CreateID(), Draggable.ToSharedPtr() );
		}
	}

	// Manually add some special cases that aren't added below
	Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryEmptyActor::StaticClass()));
	Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryCharacter::StaticClass()));
	Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryPawn::StaticClass()));
	Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryBasicShape::StaticClass(), FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicCube.ToString())), FName("ClassThumbnail.Cube"), FName("ClassIcon.Cube"), GetBasicShapeColorOverride(), TOptional<int32>(), LOCTEXT("Cube", "Cube")));
	Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryBasicShape::StaticClass(), FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicSphere.ToString())), FName("ClassThumbnail.Sphere"), FName("ClassIcon.Sphere"), GetBasicShapeColorOverride(), TOptional<int32>(), LOCTEXT("Sphere", "Sphere")));
	Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryBasicShape::StaticClass(), FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicCylinder.ToString())), FName("ClassThumbnail.Cylinder"), FName("ClassIcon.Cylinder"), GetBasicShapeColorOverride(), TOptional<int32>(), LOCTEXT("Cylinder", "Cylinder")));
	Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryBasicShape::StaticClass(), FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicCone.ToString())), FName("ClassThumbnail.Cone"), FName("ClassIcon.Cone"), GetBasicShapeColorOverride(), TOptional<int32>(), LOCTEXT("Cone", "Cone")));
	Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(*UActorFactoryBasicShape::StaticClass(), FAssetData(LoadObject<UStaticMesh>(nullptr, *UActorFactoryBasicShape::BasicPlane.ToString())), FName("ClassThumbnail.Plane"), FName("ClassIcon.Plane"), GetBasicShapeColorOverride(), TOptional<int32>(), LOCTEXT("Plane", "Plane")));

	// Below, we will be checking if the actor factory classes inherit from certain classes, but these actor factory classes might not be loaded
	// To handle this, we retrieve the derived classes via the Asset Registry
	TSet<FTopLevelAssetPath> DerivedActorClassNames, DerivedVolumeClassNames, DerivedBrushClassNames;
	AssetRegistry.GetDerivedClassNames({ AActor::StaticClass()->GetClassPathName() }, {}, DerivedActorClassNames);
	AssetRegistry.GetDerivedClassNames({ AVolume::StaticClass()->GetClassPathName() }, {}, DerivedVolumeClassNames);
	AssetRegistry.GetDerivedClassNames({ ABrush::StaticClass()->GetClassPathName() }, {}, DerivedBrushClassNames);
	
	// Make a map of UClasses to ActorFactories that support them
	const TArray< UActorFactory *>& ActorFactories = GEditor->ActorFactories;
	TMap<FTopLevelAssetPath, UActorFactory*> ActorFactoryPathMap;
	FAssetData NoAssetData;
	FText UnusedErrorMessage;
	for (int32 FactoryIdx = 0; FactoryIdx < ActorFactories.Num(); ++FactoryIdx)
	{
		UActorFactory* ActorFactory = ActorFactories[FactoryIdx];
		if (!ActorFactory)
		{
			continue;
		}
		
		FTopLevelAssetPath ActorClassPath{};
		if (ActorFactory->NewActorClass)
		{
			ActorClassPath = ActorFactory->NewActorClass->GetClassPathName();
		}
		else if (!ActorFactory->NewActorClassName.IsEmpty())
		{
			if (FPackageName::IsShortPackageName(ActorFactory->NewActorClassName))
			{
				// We cannot create a FTopLevelAssetPath from a short package name so we try resolving the name, which is what would happen if we were using
				// UActorFactory::GetDefaultActorClass. The parameters for ResolveName are the same to the ones used in StaticLoadObject, but without raising errors
				UObject* OuterPackage = nullptr;
				FString ResolvedName = ActorFactory->NewActorClassName;
				if (ResolveName(OuterPackage, ResolvedName, true, false, LOAD_NoWarn))
				{
					ActorClassPath = FTopLevelAssetPath{ OuterPackage->GetFName(), FName{ResolvedName} };
				}
			}
			else
			{
				ActorClassPath = FTopLevelAssetPath{ ActorFactory->NewActorClassName };
			}
		}

		// This is the equivalent check as the one in the loop below but using the cached data from the asset registry:
		// We only want an Actor that is not a Brush unless it is a Volume
		if (!DerivedActorClassNames.Contains(ActorClassPath) ||
			(DerivedBrushClassNames.Contains(ActorClassPath) && !DerivedVolumeClassNames.Contains(ActorClassPath)))
		{
			continue;
		}
		
		const bool bSkipARFilteredAssets = false; // Allow the retrieval of BlueprintGeneratedClasses
		FAssetData ActorClassAssetData = AssetRegistry.GetAssetByObjectPath(FSoftObjectPath{ActorClassPath}, false, bSkipARFilteredAssets);

		// Only Blueprint classes might not be loaded at this point
		int32 BPClassFlagsInt;
		if (!ActorClassAssetData.GetTagValue(FBlueprintTags::ClassFlags, BPClassFlagsInt) || ActorClassAssetData.IsAssetLoaded())
		{
			// If the class is already loaded, it will be picked up in the class iterator loop below
			ActorFactoryPathMap.Add(ActorClassPath, ActorFactory);
			continue; 
		}
			
		const EClassFlags BPClassFlags = static_cast<EClassFlags>(BPClassFlagsInt);
		if (EnumHasAllFlags(BPClassFlags, CLASS_NotPlaceable) ||
			EnumHasAnyFlags(BPClassFlags, CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			continue;
		}

		const bool bIsVolume = DerivedVolumeClassNames.Contains(ActorClassPath);
		if (bIsVolume)
		{
			// Retrieving the ActorFactory for a volume requires a call to GEditor->FindActorFactoryByClassForActorClass which requires the class
			// to be loaded to access its default actor. We therefore need to load the class here, but now that the class is loaded, we will defer
			// the processing of this Volume to the loop of loaded classes below, otherwise we would double up the entry for this AssetClass.
			const UClass* AssetClass = Cast<UClass>(ActorClassAssetData.GetAsset());
			ActorFactoryPathMap.Add(ActorClassPath, ActorFactory);
			continue; 
		}
		else if (ActorFactory && !ActorFactory->CanCreateActorFrom(NoAssetData, UnusedErrorMessage))
		{
			continue;
		}
			
		Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(ActorFactory, ActorClassAssetData));
	}

	// Add loaded classes
	for (TObjectIterator<UClass> ClassIt; ClassIt; ++ClassIt)
	{
		// Don't offer skeleton classes
		const bool bIsSkeletonClass = FKismetEditorUtilities::IsClassABlueprintSkeleton(*ClassIt);

		if (!ClassIt->HasAllClassFlags(CLASS_NotPlaceable) &&
			!ClassIt->HasAnyClassFlags(CLASS_Abstract | CLASS_Deprecated | CLASS_NewerVersionExists) &&
			ClassIt->IsChildOf(AActor::StaticClass()) &&
			(!ClassIt->IsChildOf(ABrush::StaticClass()) || ClassIt->IsChildOf(AVolume::StaticClass())) &&
			!bIsSkeletonClass)
		{
			UActorFactory* ActorFactory = ActorFactoryPathMap.FindRef(ClassIt->GetClassPathName());

			const bool bIsVolume = ClassIt->IsChildOf(AVolume::StaticClass());
			if (bIsVolume)
			{
				ActorFactory = GEditor->FindActorFactoryByClassForActorClass(UActorFactoryBoxVolume::StaticClass(), *ClassIt);
			}
			else if (ActorFactory && !ActorFactory->CanCreateActorFrom(NoAssetData, UnusedErrorMessage))
			{
				continue;
			}

			Category->Items.Add(CreateID(), MakeShared<FPlaceableItem>(ActorFactory, FAssetData(*ClassIt)));
		}
	}

	Category->Items.ValueSort([&](const TSharedPtr<FPlaceableItem>& A, const TSharedPtr<FPlaceableItem>& B) {
		return A->DisplayName.CompareTo(B->DisplayName) < 0;
		});
}

FGuid FPlacementModeModule::CreateID()
{
	return FGuid::NewGuid();
}

FPlacementModeID FPlacementModeModule::CreateID(FName InCategory)
{
	FPlacementModeID NewID;
	NewID.UniqueID = CreateID();
	NewID.Category = InCategory;
	return NewID;
}

bool FPlacementModeModule::PassesFilters(const TSharedPtr<FPlaceableItem>& Item) const
{
	if (PlaceableItemPredicates.Num() == 0)
	{
		return true;
	}

	for (auto& PredicatePair : PlaceableItemPredicates)
	{
		if (PredicatePair.Value(Item))
		{
			bool bPlaceable = true;
			UClass* AssetClass = Item->AssetData.GetClass();
			if (AssetClass == UClass::StaticClass())
			{
				UClass* Class = Cast<UClass>(Item->AssetData.GetAsset());

				bPlaceable = AssetSelectionUtils::IsClassPlaceable(Class);
			}
			else if (AssetClass && AssetClass->IsChildOf<UBlueprint>())
			{
				// For blueprints, attempt to determine placeability from its tag information

				FString TagValue;

				if (Item->AssetData.GetTagValue(FBlueprintTags::NativeParentClassPath, TagValue) && !TagValue.IsEmpty())
				{
					// If the native parent class can't be placed, neither can the blueprint
					UClass* NativeParentClass = UClass::TryFindTypeSlow<UClass>(FPackageName::ExportTextPathToObjectPath(TagValue));

					bPlaceable = AssetSelectionUtils::IsChildBlueprintPlaceable(NativeParentClass);
				}

				if (bPlaceable && Item->AssetData.GetTagValue(FBlueprintTags::ClassFlags, TagValue) && !TagValue.IsEmpty())
				{
					// Check to see if this class is placeable from its class flags

					const int32 NotPlaceableFlags = CLASS_NotPlaceable | CLASS_Deprecated | CLASS_Abstract;
					uint32 ClassFlags = FCString::Atoi(*TagValue);

					bPlaceable = (ClassFlags & NotPlaceableFlags) == CLASS_None;
				}
			}
			return bPlaceable;
		}
	}
	return false;
}

void FPlacementModeModule::OnCategoryPermissionListChanged()
{
	PlacementModeCategoryListChanged.Broadcast();
}

#undef LOCTEXT_NAMESPACE
