// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPlacementModeTools.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Styling/AppStyle.h"
#include "EditorModeManager.h"
#include "EditorModes.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementObjectInterface.h"
#include "Framework/Application/SlateApplication.h"
#include "AssetThumbnail.h"
#include "LevelEditor.h"
#include "LevelEditorActions.h"
#include "LevelEditorViewport.h"
#include "LevelEditorSubsystem.h"
#include "ContentBrowserDataDragDropOp.h"
#include "EditorClassUtils.h"
#include "Widgets/Input/SSearchBox.h"
#include "ClassIconFinder.h"
#include "Widgets/Docking/SDockTab.h"
#include "ThumbnailRendering/ThumbnailManager.h"
#include "AssetSelection.h"
#include "SAssetDropTarget.h"
#include "ActorFactories/ActorFactory.h"
#include "ScopedTransaction.h"
#include "Layout/CategoryDrivenContentBuilder.h"
#include "ToolkitBuilder.h"
#include "Styles/SlateBrushTemplates.h"
#include "Layout/WidgetPath.h"
#include "Styling/CoreStyle.h"
#include "IDocumentation.h"

#define LOCTEXT_NAMESPACE "PlacementMode"


namespace PlacementModeTools
{
	bool bItemInternalsInTooltip = false;
	FAutoConsoleVariableRef CVarItemInternalsInTooltip(TEXT("PlacementMode.ItemInternalsInTooltip"), bItemInternalsInTooltip, TEXT("Shows placeable item internal information in its tooltip"));

	bool bAllowAssetFactoryInterfaceWorkflowForItems = true;
	FAutoConsoleVariableRef CVarAllowAssetFactoryInterfaceWorkflowForItems(
		TEXT("PlacementMode.AllowAssetFactoryInterfaceWorkflowForItems"), 
		bAllowAssetFactoryInterfaceWorkflowForItems,
		TEXT("If enabled AssetFactoryInterface would be considered when placing actors and not always the default ActorFactory would be used.")
	);
}

struct FSortPlaceableItems
{
	static bool SortItemsByOrderThenName(const TSharedPtr<FPlaceableItem>& A, const TSharedPtr<FPlaceableItem>& B)
	{
		if (A->SortOrder.IsSet())
		{
			if (B->SortOrder.IsSet())
			{
				return A->SortOrder.GetValue() < B->SortOrder.GetValue();
			}
			else
			{
				return true;
			}
		}
		else if (B->SortOrder.IsSet())
		{
			return false;
		}
		else
		{
			return SortItemsByName(A, B);
		}
	}

	static bool SortItemsByName(const TSharedPtr<FPlaceableItem>& A, const TSharedPtr<FPlaceableItem>& B)
	{
		return A->DisplayName.CompareTo(B->DisplayName) < 0;
	}
};

namespace PlacementViewFilter
{
	void GetBasicStrings(const FPlaceableItem& InPlaceableItem, TArray<FString>& OutBasicStrings)
	{
		OutBasicStrings.Add(InPlaceableItem.DisplayName.ToString());

		if (!InPlaceableItem.NativeName.IsEmpty())
		{
			OutBasicStrings.Add(InPlaceableItem.NativeName);
		}

		const FString* SourceString = FTextInspector::GetSourceString(InPlaceableItem.DisplayName);
		if (SourceString)
		{
			OutBasicStrings.Add(*SourceString);
		}
	}
} // namespace PlacementViewFilter


template <>
struct TWidgetTypeTraits<class SPlacementAssetThumbnail>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * These are the asset thumbnails.
 */
class SPlacementAssetThumbnail : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPlacementAssetThumbnail )
		: _Width( 32 )
		, _Height( 32 )
		, _AlwaysUseGenericThumbnail( false )
		, _AssetTypeColorOverride()
		, _CustomIconBrush( nullptr )
	{}

	SLATE_ARGUMENT( uint32, Width )

	SLATE_ARGUMENT( uint32, Height )

	SLATE_ARGUMENT( FName, ClassThumbnailBrushOverride )

	SLATE_ARGUMENT( bool, AlwaysUseGenericThumbnail )

	SLATE_ARGUMENT( TOptional<FLinearColor>, AssetTypeColorOverride )
	
	SLATE_ARGUMENT( const FSlateBrush*, CustomIconBrush )

	SLATE_END_ARGS()

	void Construct( const FArguments& InArgs, const FAssetData& InAsset)
	{
		Asset = InAsset;

		TSharedPtr<FAssetThumbnailPool> ThumbnailPool = UThumbnailManager::Get().GetSharedThumbnailPool();

		Thumbnail = MakeShareable(new FAssetThumbnail(Asset, InArgs._Width, InArgs._Height, ThumbnailPool));
		
		TSharedPtr<SImage> ThumbnailImage;

		// figure out the proper image to show based on whether the asset is a class type
		TWeakObjectPtr<UClass> ThumbnailClass = MakeWeakObjectPtr( const_cast<UClass*>( FClassIconFinder::GetIconClassForAssetData( Asset, &bIsClassType ) ) );
		const FName AssetClassName = Asset.AssetClassPath.GetAssetName();
		const FName DefaultThumbnail = bIsClassType ? NAME_None : FName( *FString::Printf( TEXT("ClassThumbnail.%s"), *AssetClassName.ToString() ) );
		const FSlateBrush* ThumbnailBrush = !InArgs._ClassThumbnailBrushOverride.IsNone() ?
			FClassIconFinder::FindThumbnailForClass( nullptr,  InArgs._ClassThumbnailBrushOverride ) :
			FClassIconFinder::FindThumbnailForClass( ThumbnailClass.Get(), DefaultThumbnail );

		if ( InArgs._CustomIconBrush )
		{
			ThumbnailBrush = InArgs._CustomIconBrush;
		}

		ChildSlot[ SAssignNew( ThumbnailImage, SImage ).Image( ThumbnailBrush ) ];
	}

private:

	FAssetData Asset;
	TSharedPtr< FAssetThumbnail > Thumbnail;

	/** Indicates whether the Asset is a class type */
	bool bIsClassType;
};

void SPlacementAssetEntry::Construct(const FArguments& InArgs, const TSharedPtr<const FPlaceableItem>& InItem)
{	
	OnGetMenuContent = InArgs._OnGetMenuContent;
	bIsPressed = false;

	Item = InItem;

	TSharedPtr< SHorizontalBox > ActorType = SNew( SHorizontalBox );

	const bool bIsClass = Item->AssetData.GetClass() == UClass::StaticClass();
	const bool bIsActor = bIsClass ? CastChecked<UClass>(Item->AssetData.GetAsset())->IsChildOf(AActor::StaticClass()) : false;

	AActor* DefaultActor = nullptr;
	if (Item->Factory != nullptr)
	{
		DefaultActor = Item->Factory->GetDefaultActor(Item->AssetData);
	}
	else if (bIsActor)
	{
		DefaultActor = CastChecked<AActor>(CastChecked<UClass>(Item->AssetData.GetAsset())->GetDefaultObject(false));
	}

	TSharedPtr<IToolTip> AssetEntryToolTip;
	if (PlacementModeTools::bItemInternalsInTooltip)
	{
		AssetEntryToolTip = FSlateApplicationBase::Get().MakeToolTip(
			FText::Format(LOCTEXT("ItemInternalsTooltip", "Native Name: {0}\nAsset Path: {1}\nFactory Class: {2}"), 
			FText::FromString(Item->NativeName), 
			FText::FromString(Item->AssetData.GetObjectPathString()),
			FText::FromString(Item->Factory ? Item->Factory->GetClass()->GetName() : TEXT("None"))));
	}

	UClass* DocClass = nullptr;
	if(DefaultActor != nullptr)
	{
		DocClass = DefaultActor->GetClass();
		if (!AssetEntryToolTip)
		{
			AssetEntryToolTip = FEditorClassUtils::GetTooltip(DefaultActor->GetClass());
		}
	}

	if (!AssetEntryToolTip)
	{
		AssetEntryToolTip = IDocumentation::Get()->CreateToolTip(Item->DisplayName, nullptr, "Shared/Types/AssetEntries", Item->DisplayName.ToString());
	}

	const FButtonStyle& ButtonStyle = FAppStyle::GetWidgetStyle<FButtonStyle>( "PlacementBrowser.Asset" );

	NormalImage = &ButtonStyle.Normal;
	HoverImage = &ButtonStyle.Hovered;
	PressedImage = &ButtonStyle.Pressed;
	float ThumbnailBoxWidth = 40;
	
	float TextFillWidth = 0.99f;
	const FMargin DragHandlePadding{ 0.f,0.f,8.f, 0.f };
	
	FMargin WholeAssetPadding{ 8.f, 2.f, 12.f, 2.f };
	const FSlateBrush* WholeAssetBackgroundBrush{ FAppStyle::Get().GetBrush( "PlacementBrowser.Asset.Background" ) };
	FMargin ThumbnailBoxPadding{ 8.f ,4.f,8.f, 4.f };
	FMargin AssetTextPadding{ 9, 0, 0, 1 };
	TSharedRef<SWidget> DraggableAssetEndWidget = SNullWidget::NullWidget;

	WholeAssetPadding = 0;
	WholeAssetBackgroundBrush = FAppStyle::Get().GetBrush("PlacementBrowser.Asset.ThumbnailBackground");
	ThumbnailBoxPadding = FMargin{ 4.f,4.f,0.f, 4.f };
	AssetTextPadding = FMargin{ 4, 0, 8, 1 };
	DraggableAssetEndWidget = SNew(SBox )
								.Padding( DragHandlePadding )
								[  SNew(SImage).Image( FAppStyle::GetBrush("VerticalBoxDragIndicatorShort") ) ];
	ThumbnailBoxWidth = 20;
	
	const FSlateBrush* CustomIconBrush = nullptr;
	if ( Item->DragHandler.IsValid() && Item->DragHandler->IconBrush )
	{
		CustomIconBrush = Item->DragHandler->IconBrush;
	}
	
	ChildSlot
	.Padding( WholeAssetPadding )
	[

		SNew(SOverlay)

		+SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage( WholeAssetBackgroundBrush)
			.Cursor( EMouseCursor::GrabHand )
			.ToolTip( AssetEntryToolTip )
			.Padding(0)
			[

				SNew( SHorizontalBox )

				+ SHorizontalBox::Slot()
				.Padding( ThumbnailBoxPadding )
				.AutoWidth()
				.HAlign(HAlign_Center)
				.VAlign(VAlign_Center)
				[
					SNew( SBox )
					.WidthOverride( ThumbnailBoxWidth )
					.HeightOverride(40)
					[
						SNew( SPlacementAssetThumbnail, Item->AssetData )
						.ClassThumbnailBrushOverride( Item->ClassThumbnailBrushOverride )
						.AlwaysUseGenericThumbnail( Item->bAlwaysUseGenericThumbnail )
						.AssetTypeColorOverride( FLinearColor::Transparent )
						.CustomIconBrush( CustomIconBrush )
					]
				]

				+ SHorizontalBox::Slot()
				.VAlign(VAlign_Fill)
				.Padding(0)
				[

					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("PlacementBrowser.Asset.LabelBack"))
					[
						SNew( SHorizontalBox)
						+SHorizontalBox::Slot()	
						.FillContentWidth( TextFillWidth )
						.Padding( AssetTextPadding )
						.VAlign(VAlign_Center)
						[
							SNew( STextBlock )
							.TextStyle( FAppStyle::Get(), "PlacementBrowser.Asset.Name" )
							.Text( Item->DisplayName )
							.OverflowPolicy( ETextOverflowPolicy::Ellipsis )
							.HighlightText(  InArgs._HighlightText )
						]
						+ SHorizontalBox::Slot()
								.VAlign( VAlign_Center )
								.AutoWidth()
								[ DraggableAssetEndWidget ]
					]
				]
			]
		]

		+SOverlay::Slot()
		[
			SNew(SBorder)
			.BorderImage( this, &SPlacementAssetEntry::GetBorder )
			.Cursor( EMouseCursor::GrabHand )
			.ToolTip( AssetEntryToolTip )
		]
	];
}

FReply SPlacementAssetEntry::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = true;

		return FReply::Handled().DetectDrag( SharedThis( this ), MouseEvent.GetEffectingButton() );
	}

	// Create the context menu to be launched on right mouse click. 
	if ( MouseEvent.GetEffectingButton() == EKeys::RightMouseButton )
	{
		FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();

		FSlateApplication::Get().PushMenu(
			AsShared(),
			WidgetPath,
			OnGetMenuContent.IsBound() ? OnGetMenuContent.Execute() : SNullWidget::NullWidget,
			MouseEvent.GetScreenSpacePosition(),
			FPopupTransitionEffect( FPopupTransitionEffect::ContextMenu )
			);

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SPlacementAssetEntry::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = false;
	}

	return FReply::Unhandled();
}

FReply SPlacementAssetEntry::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bIsPressed = false;

	if (FEditorDelegates::OnAssetDragStarted.IsBound())
	{
		TArray<FAssetData> DraggedAssetDatas;
		DraggedAssetDatas.Add( Item->AssetData );
		FEditorDelegates::OnAssetDragStarted.Broadcast( DraggedAssetDatas, Item->Factory );
		return FReply::Handled();
	}

	if ( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) )
	{
		if ( Item->DragHandler.IsValid() && Item->DragHandler->GetContentToDrag.IsBound() )
		{
			return FReply::Handled().BeginDragDrop( Item->DragHandler->GetContentToDrag.Execute() );
		}
		
		return FReply::Handled().BeginDragDrop(FAssetDragDropOp::New(Item->AssetData, Item->AssetFactory));
	}
	else
	{
		return FReply::Handled();
	}
}

bool SPlacementAssetEntry::IsPressed() const
{
	return bIsPressed;
}

const FSlateBrush* SPlacementAssetEntry::GetBorder() const
{
	if ( IsPressed() )
	{
		return PressedImage;
	}
	else if ( IsHovered() )
	{
		return HoverImage;
	}
	else
	{
		return NormalImage;
	}
}


void SPlacementAssetMenuEntry::Construct(const FArguments& InArgs, const TSharedPtr<const FPlaceableItem>& InItem)
{	
	bIsPressed = false;

	check(InItem.IsValid());

	Item = InItem;

	AssetImage = nullptr;

	TSharedPtr< SHorizontalBox > ActorType = SNew( SHorizontalBox );

	const bool bIsClass = Item->AssetData.GetClass() == UClass::StaticClass();
	const bool bIsActor = bIsClass ? CastChecked<UClass>(Item->AssetData.GetAsset())->IsChildOf(AActor::StaticClass()) : false;

	AActor* DefaultActor = nullptr;
	if (Item->Factory != nullptr)
	{
		DefaultActor = Item->Factory->GetDefaultActor(Item->AssetData);
	}
	else if (bIsActor)
	{
		DefaultActor = CastChecked<AActor>(CastChecked<UClass>(Item->AssetData.GetAsset())->GetDefaultObject(false));
	}

	UClass* DocClass = nullptr;
	TSharedPtr<IToolTip> AssetEntryToolTip;
	if(DefaultActor != nullptr)
	{
		DocClass = DefaultActor->GetClass();
		AssetEntryToolTip = FEditorClassUtils::GetTooltip(DefaultActor->GetClass());
	}

	if (!AssetEntryToolTip.IsValid())
	{
		AssetEntryToolTip = IDocumentation::Get()->CreateToolTip(Item->DisplayName, nullptr, "Shared/Types/AssetEntries", Item->DisplayName.ToString());
	}
	
	const FButtonStyle& ButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>( "Menu.Button" );
	const float MenuIconSize = FAppStyle::Get().GetFloat("Menu.MenuIconSize");

	Style = &ButtonStyle;

	// Create doc link widget if there is a class to link to
	TSharedRef<SWidget> DocWidget = SNew(SSpacer);
	if(DocClass != NULL)
	{
		DocWidget = FEditorClassUtils::GetDocumentationLinkWidget(DocClass);
		DocWidget->SetCursor( EMouseCursor::Default );
	}

	ChildSlot
	.HAlign(HAlign_Fill)
	.VAlign(VAlign_Fill)
	[
		SNew(SBorder)
		.BorderImage( this, &SPlacementAssetMenuEntry::GetBorder )
		.Cursor( EMouseCursor::GrabHand )
		.ToolTip( AssetEntryToolTip )
		.Padding(FMargin(10.f, 3.f, 5.f, 3.f))
		[
			SNew( SHorizontalBox )

			+ SHorizontalBox::Slot()
			.Padding(14.0f, 0.f, 10.f, 0.0f)
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SBox)
				.WidthOverride(MenuIconSize)
				.HeightOverride(MenuIconSize)
				[
					SNew(SImage)
					.Image(this, &SPlacementAssetMenuEntry::GetIcon)
					.ColorAndOpacity(FSlateColor::UseSubduedForeground())
				]
			]

			+ SHorizontalBox::Slot()
			.FillWidth(1.f)
			.Padding(1.f, 0.f, 0.f, 0.f)
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			[

				SNew( STextBlock )
				.ColorAndOpacity(FSlateColor::UseForeground())
				.Text( Item->DisplayName )
			]


			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.AutoWidth()
			[
				SNew(SImage)
                .ColorAndOpacity(FSlateColor::UseSubduedForeground())
				.Image(FAppStyle::Get().GetBrush("Icons.DragHandle"))
			]
		]
	];
}


const FSlateBrush* SPlacementAssetMenuEntry::GetIcon() const
{
	if (AssetImage != nullptr)
	{
		return AssetImage;
	}

	if (Item->DragHandler && Item->DragHandler->IconBrush)
	{
		AssetImage = Item->DragHandler->IconBrush;
	}
	else if (Item->ClassIconBrushOverride != NAME_None)
	{
		AssetImage = FSlateIconFinder::FindCustomIconBrushForClass(nullptr, TEXT("ClassIcon"), Item->ClassIconBrushOverride);
	}
	else
	{
		AssetImage = FSlateIconFinder::FindIconBrushForClass(FClassIconFinder::GetIconClassForAssetData(Item->AssetData));
	}

	return AssetImage;
}

bool SPlacementAssetMenuEntry::TryPlacingItemInLevel(TNotNull<ULevel*> DesiredLevel, AActor*& OutPlacedActor)
{
	bool bPlacementHandled = false;
	if (TScriptInterface<IAssetFactoryInterface> AssetFactory = Item->AssetFactory)
	{
		if (UObject* LoadedAsset = Item->AssetData.GetAsset())
		{
			UE::AssetPlacementUtil::FExtraPlaceAssetOptions PlacementOptions;
			PlacementOptions.bSelectOutput = true;
			PlacementOptions.ObjectFlags = RF_Transactional;
			PlacementOptions.FactoryToUse = AssetFactory;
			PlacementOptions.Name = NAME_None;

			TArray<FTypedElementHandle> PlacedItemsHandles = FLevelEditorViewportClient::TryPlacingAssetObject(DesiredLevel, LoadedAsset, PlacementOptions);
			bPlacementHandled = !PlacedItemsHandles.IsEmpty();

			// Placement succeeded. Not necessarily an actor, but could be elements.
			if (bPlacementHandled)
			{
				for (const FTypedElementHandle& Element : PlacedItemsHandles)
				{
					if (AActor* Actor = ActorElementDataUtil::GetActorFromHandle(Element))
					{
						OutPlacedActor = Actor; 
						break;
					}
				}

				if (IPlacementModeModule::IsAvailable())
				{
					IPlacementModeModule::Get().AddToRecentlyPlaced(LoadedAsset, AssetFactory);
				}
			}
		}
	}

	return bPlacementHandled;
}


FReply SPlacementAssetMenuEntry::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = true;

		return FReply::Handled().DetectDrag( SharedThis( this ), MouseEvent.GetEffectingButton() );
	}

	return FReply::Unhandled();
}

FReply SPlacementAssetMenuEntry::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if ( MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton )
	{
		bIsPressed = false;

		{
			// Note: Capture the add and the move within a single transaction, so that the placed actor position is calculated correctly by the transaction diff
			FScopedTransaction Transaction(NSLOCTEXT("UnrealEd", "CreateActor", "Create Actor"));
			
			AActor* NewActor = nullptr;
			bool bPlacementHandled = false;
			if (PlacementModeTools::bAllowAssetFactoryInterfaceWorkflowForItems)
			{
				if (ULevelEditorSubsystem* LevelEditorSubsystem = GEditor->GetEditorSubsystem<ULevelEditorSubsystem>())
				{
					if (ULevel* DesiredLevel = LevelEditorSubsystem->GetCurrentLevel())
					{
						bPlacementHandled = TryPlacingItemInLevel(DesiredLevel, NewActor);
					}
				}
			}
			
			// Likely not an element placement, try adding actor instead.
			if (!bPlacementHandled)
			{
				NewActor = FLevelEditorActionCallbacks::AddActor(Item->Factory, Item->AssetData, nullptr);
			}

			if (NewActor && GCurrentLevelEditingViewportClient)
			{
				GEditor->MoveActorInFrontOfCamera(*NewActor,
					GCurrentLevelEditingViewportClient->GetViewLocation(),
					GCurrentLevelEditingViewportClient->GetViewRotation().Vector()
				);
			}
		}

		if (!MouseEvent.IsControlDown())
		{
			FSlateApplication::Get().DismissAllMenus();
		}

		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SPlacementAssetMenuEntry::OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	bIsPressed = false;

	if (FEditorDelegates::OnAssetDragStarted.IsBound())
	{
		TArray<FAssetData> DraggedAssetDatas;
		DraggedAssetDatas.Add( Item->AssetData );
		FEditorDelegates::OnAssetDragStarted.Broadcast( DraggedAssetDatas, Item->Factory );
		return FReply::Handled();
	}

	if( MouseEvent.IsMouseButtonDown( EKeys::LeftMouseButton ) )
	{
		if (Item->DragHandler.IsValid() && Item->DragHandler->GetContentToDrag.IsBound())
		{
			return FReply::Handled().BeginDragDrop( Item->DragHandler->GetContentToDrag.Execute() );
		}

		return FReply::Handled().BeginDragDrop(FAssetDragDropOp::New(Item->AssetData, Item->AssetFactory));
	}
	else
	{
		return FReply::Handled();
	}
}

bool SPlacementAssetMenuEntry::IsPressed() const
{
	return bIsPressed;
}

const FSlateBrush* SPlacementAssetMenuEntry::GetBorder() const
{
	if ( IsPressed() )
	{
		return &(Style->Pressed);
	}
	else if ( IsHovered() )
	{
		return &(Style->Hovered);
	}
	else
	{
		return &(Style->Normal);
	}
}

FSlateColor SPlacementAssetMenuEntry::GetForegroundColor() const
{
	if (IsPressed())
	{
		return Style->PressedForeground;
	}
	else if (IsHovered())
	{
		return Style->HoveredForeground;
	}
	else
	{
		return Style->NormalForeground;
	}
}


SPlacementModeTools::~SPlacementModeTools()
{
	if ( IPlacementModeModule::IsAvailable() )
	{
		IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
		PlacementModeModule.OnRecentlyPlacedChanged().RemoveAll(this);
		PlacementModeModule.OnAllPlaceableAssetsChanged().RemoveAll(this);
		PlacementModeModule.OnPlacementModeCategoryListChanged().RemoveAll(this);
		PlacementModeModule.OnPlaceableItemFilteringChanged().RemoveAll(this);
	}
}

void SPlacementModeTools::Construct( const FArguments& InArgs, TSharedRef<SDockTab> ParentTab )
{
	bRefreshAllClasses = false;
	bRefreshRecentlyPlaced = false;
	bUpdateShownItems = true;
	bIsRawSearchChange = false;

	FCategoryDrivenContentBuilderArgs Args( "PlacementModes", UE::DisplayBuilders::FBuilderKeys::Get().PlaceActors() );
	Args.FavoritesCommandName = FBuiltInPlacementCategories::Favorites();
	Args.ActiveCategoryName = FBuiltInPlacementCategories::Basic();

	CategoryContentBuilder = MakeShared<FCategoryDrivenContentBuilder>( Args );
	CategoryContentBuilder->UpdateContentForCategoryDelegate.BindSP( SharedThis( this ), &SPlacementModeTools::UpdateContentForCategory );

	ActiveTabName = FBuiltInPlacementCategories::Basic();

	ParentTab->SetOnTabDrawerOpened(FSimpleDelegate::CreateSP(this, &SPlacementModeTools::OnTabDrawerOpened));

	SearchTextFilter = MakeShareable(new FPlacementAssetEntryTextFilter(
		FPlacementAssetEntryTextFilter::FItemToStringArray::CreateStatic(&PlacementViewFilter::GetBasicStrings)
		));

	UpdatePlacementCategories();

	TSharedRef<SScrollBar> ScrollBar = SNew(SScrollBar)
		.Thickness(FVector2D(9.0f, 9.0f));

	ChildSlot
	[
		SNew( SVerticalBox )

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(8)
			[
				SAssignNew( SearchBoxPtr, SSearchBox )
				.HintText(LOCTEXT("SearchPlaceables", "Search Classes"))
				.OnTextChanged(this, &SPlacementModeTools::OnSearchChanged)
				.OnTextCommitted(this, &SPlacementModeTools::OnSearchCommitted)
			]
		]
		+ SVerticalBox::Slot()
			.FillHeight(1)
			[
				SNew(SBorder)
				.BorderImage( FAppStyle::GetBrush("Brushes.Panel") )
				.Padding(0)
				[
					CategoryContentBuilder->GenerateWidgetSharedRef()
				]
			]
	];

	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();
	PlacementModeModule.OnRecentlyPlacedChanged().AddSP(this, &SPlacementModeTools::RequestRefreshRecentlyPlaced);
	PlacementModeModule.OnAllPlaceableAssetsChanged().AddSP(this, &SPlacementModeTools::RequestRefreshAllClasses);
	PlacementModeModule.OnPlaceableItemFilteringChanged().AddSP(this, &SPlacementModeTools::RequestUpdateShownItems);
	PlacementModeModule.OnPlacementModeCategoryListChanged().AddSP(this, &SPlacementModeTools::UpdatePlacementCategories);
	PlacementModeModule.OnPlacementModeCategoryRefreshed().AddSP(this, &SPlacementModeTools::OnCategoryRefresh);
}

FName SPlacementModeTools::GetActiveTab() const
{
	return IsSearchActive() ? FBuiltInPlacementCategories::AllClasses() : ActiveTabName;
}

void SPlacementModeTools::SetActiveTab(FName TabName)
{
	if (TabName != ActiveTabName)
	{
		ActiveTabName = TabName;
		IPlacementModeModule::Get().RegenerateItemsForCategory(ActiveTabName);
	}
}

void SPlacementModeTools::UpdateShownItems()
{
	bUpdateShownItems = false;

	IPlacementModeModule& PlacementModeModule = IPlacementModeModule::Get();

	const FPlacementCategoryInfo* Category = PlacementModeModule.GetRegisteredPlacementCategory(GetActiveTab());
	if (!Category)
	{
		return;
	}
	else if (Category->CustomGenerator && Category->CustomDraggableItems.IsEmpty())
	{
		CategoryContentBuilder->FillWithBuilder( Category->CustomGenerator() );
	}
	else if ( IsFavoritesCategorySelected() )
	{
		IPlacementModeModule::Get().RegenerateItemsForCategory( FBuiltInPlacementCategories::AllClasses() );
		PlacementModeModule.GetItemsWithNamesForCategory( FBuiltInPlacementCategories::AllClasses(), FavoriteItems, CategoryContentBuilder->GetFavorites() );
	}
	else
	{
		FilteredItems.Reset();

		if (IsSearchActive())
		{
			auto Filter = [&](const TSharedPtr<FPlaceableItem>& Item) { return SearchTextFilter->PassesFilter(*Item); };
			PlacementModeModule.GetFilteredItemsForCategory(Category->UniqueHandle, FilteredItems, Filter);
			
			if (Category->bSortable)
			{
				FilteredItems.Sort(&FSortPlaceableItems::SortItemsByName);
			}
		}
		else
		{
			if ( !Category->CustomDraggableItems.IsEmpty() )
			{
				for (TSharedRef<FPlaceableItem> Item : Category->CustomDraggableItems)
				{
					FilteredItems.Add( Item.ToSharedPtr() );
				}
			}
			else
			{
				PlacementModeModule.GetItemsForCategory(Category->UniqueHandle, FilteredItems);
			}

			if (Category->bSortable)
			{
				// The item order makes sense internally to a category, not across all classes, so sort by name only in the all classes case
				if (Category->UniqueHandle == FBuiltInPlacementCategories::AllClasses())
				{
					FilteredItems.Sort(&FSortPlaceableItems::SortItemsByName);
				}
				else
				{
					FilteredItems.Sort(&FSortPlaceableItems::SortItemsByOrderThenName);
				}
			}
		}
	}
}

bool SPlacementModeTools::IsSearchActive() const
{
	return !SearchTextFilter->GetRawFilterText().IsEmpty();
}

ECheckBoxState SPlacementModeTools::GetPlacementTabCheckedState( FName CategoryName ) const
{
	return ActiveTabName == CategoryName ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

TSharedRef<SWidget> SPlacementModeTools::GetPlacementAssetWidget( const TSharedPtr<FPlaceableItem>& InItem ) const
{
	TSharedRef<SPlacementAssetEntry> Entry =  SNew( SPlacementAssetEntry, InItem.ToSharedRef() )
		.HighlightText(this, &SPlacementModeTools::GetHighlightText)
		.Clipping( EWidgetClipping::ClipToBounds )
		.OnGetMenuContent_Lambda( [this, &InItem] ()
		{
			return CategoryContentBuilder.IsValid() ? CategoryContentBuilder->CreateFavoritesContextMenu( InItem->NativeName ) : SNullWidget::NullWidget;
		});
	return Entry;
}

void SPlacementModeTools::UpdateContentForCategory( FName CategoryName, FText CategoryLabel )
{
	SetActiveTab( CategoryName );
	FavoriteItems.Empty();

	CategoryContentBuilder->ClearCategoryContent();

	// if the Category name is not none, the user updated the category, so clear out the search ~ the Category choice should override it.
	// The call of UpdateShownItems below will update search state based on this setting.
	if ( !CategoryName.IsNone() )
	{
		TGuardValue<bool> IsRawSearchChangeGuard(bIsRawSearchChange, true);
		SearchBoxPtr->SetText( FText::GetEmpty() );
	}

	UpdateShownItems();

	const FPlacementCategoryInfo* Category = IPlacementModeModule::Get().GetRegisteredPlacementCategory( CategoryName );
	if ( Category && Category->CustomGenerator && Category->CustomDraggableItems.IsEmpty() )
	{
		CategoryContentBuilder->FillWithBuilder( Category->CustomGenerator() );
		return;
	}

	if ( IsFavoritesCategorySelected() )
	{
		for ( const TSharedPtr<FPlaceableItem>& Item : FavoriteItems )
		{
			CategoryContentBuilder->AddBuilder( GetPlacementAssetWidget( Item ) );
		}
	}
	else
	{
		for (const TSharedPtr<FPlaceableItem>& Item : FilteredItems)
		{
			CategoryContentBuilder->AddBuilder( GetPlacementAssetWidget(Item) );
		}
	}
}

bool SPlacementModeTools::IsFavoritesCategorySelected() const
{
	return ActiveTabName == FBuiltInPlacementCategories::Favorites() && !IsSearchActive();
}

void SPlacementModeTools::OnCategoryChanged(const ECheckBoxState NewState, FName InCategory)
{
	if (NewState == ECheckBoxState::Checked)
	{
		SetActiveTab(InCategory);
	}
}

void SPlacementModeTools::OnTabDrawerOpened()
{
	FSlateApplication::Get().SetKeyboardFocus(SearchBoxPtr, EFocusCause::SetDirectly);
}

void SPlacementModeTools::RequestUpdateShownItems()
{
	bUpdateShownItems = true;
}

void SPlacementModeTools::RequestRefreshRecentlyPlaced( const TArray< FActorPlacementInfo >& RecentlyPlaced )
{
	if (GetActiveTab() == FBuiltInPlacementCategories::RecentlyPlaced())
	{
		bRefreshRecentlyPlaced = true;
	}
}

void SPlacementModeTools::RequestRefreshAllClasses()
{
	if (GetActiveTab() == FBuiltInPlacementCategories::AllClasses())
	{
		bRefreshAllClasses = true;
	}
}

void SPlacementModeTools::OnCategoryRefresh(FName CategoryName)
{
	if (GetActiveTab() == CategoryName)
	{
		RequestUpdateShownItems();
	}
}

void SPlacementModeTools::UpdatePlacementCategories()
{
	bool bBasicTabExists = false;
	FName TabToActivate;

	TArray<FPlacementCategoryInfo> Categories;
	IPlacementModeModule::Get().GetSortedCategories(Categories);

	TArray<UE::DisplayBuilders::FBuilderInput> BuilderInputArray;

	for (const FPlacementCategoryInfo& Category : Categories)
	{
		UE::DisplayBuilders::FBuilderInput InputInfo = UE::DisplayBuilders::FBuilderInput( Category.UniqueHandle, Category.DisplayName,
			Category.DisplayIcon, EUserInterfaceActionType::ToggleButton );

		if (!Category.ShortDisplayName.IsEmpty())
		{
			InputInfo.ButtonArgs.LabelOverride = Category.ShortDisplayName;
		}

		BuilderInputArray.Add( InputInfo );
		
		if (Category.UniqueHandle == FBuiltInPlacementCategories::Basic())
		{
			bBasicTabExists = true;
		}

		if (Category.UniqueHandle == ActiveTabName)
		{
			TabToActivate = ActiveTabName;
		}
	}
	CategoryContentBuilder->InitializeCategoryButtons( BuilderInputArray );
	
	if (TabToActivate.IsNone())
	{
		if (bBasicTabExists)
		{
			TabToActivate = FBuiltInPlacementCategories::Basic();
		}
		else if (Categories.Num() > 0)
		{
			TabToActivate = Categories[0].UniqueHandle;
		}
	}

	SetActiveTab(TabToActivate);
}

void SPlacementModeTools::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	if (bRefreshAllClasses)
	{
		IPlacementModeModule::Get().RegenerateItemsForCategory(FBuiltInPlacementCategories::AllClasses());
		bRefreshAllClasses = false;
	}

	if (bRefreshRecentlyPlaced)
	{
		IPlacementModeModule::Get().RegenerateItemsForCategory(FBuiltInPlacementCategories::RecentlyPlaced());
		bRefreshRecentlyPlaced = false;
	}

	if (bUpdateShownItems)
	{
		UpdateShownItems();
	}
}

void SPlacementModeTools::OnSearchChanged(const FText& InFilterText)
{
	// If the search text was previously empty we do a full rebuild of our cached widgets
	// for the placeable widgets.
	if ( !IsSearchActive() )
	{
		bRefreshAllClasses = true;
	}
	else
	{
		bUpdateShownItems = true;
	}

	const FText OldText = SearchTextFilter->GetRawFilterText();
	SearchTextFilter->SetRawFilterText( InFilterText );
	SearchBoxPtr->SetError( SearchTextFilter->GetFilterErrorText() );

	if ( !OldText.EqualToCaseIgnored( InFilterText ) && !bIsRawSearchChange )
	{
		CategoryContentBuilder->SetShowNoCategorySelection( IsSearchActive() );
		CategoryContentBuilder->UpdateWidget();
	}
}

void SPlacementModeTools::OnSearchCommitted(const FText& InFilterText, ETextCommit::Type InCommitType)
{
	OnSearchChanged(InFilterText);
}

FText SPlacementModeTools::GetHighlightText() const
{
	return SearchTextFilter->GetRawFilterText();
}

#undef LOCTEXT_NAMESPACE