// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Layout/Visibility.h"
#include "Input/Reply.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SWidget.h"
#include "Widgets/SCompoundWidget.h"
#include "ActorPlacementInfo.h"
#include "IPlacementModeModule.h"
#include "Widgets/Layout/SUniformWrapPanel.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Misc/TextFilter.h"

class ULevel;
class STextBlock;
class FCategoryDrivenContentBuilder;

template <>
struct TWidgetTypeTraits<class SPlacementAssetEntry>
{
	static constexpr bool SupportsInvalidation() { return true; }
};

/**
 * A tile representation of the class or the asset.  These are embedded into the views inside
 * of each tab.
 */
class SPlacementAssetEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPlacementAssetEntry){}

		/** Highlight this text in the text block */
		SLATE_ATTRIBUTE(FText, HighlightText)
		SLATE_EVENT( FOnGetContent, OnGetMenuContent )

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<const FPlaceableItem>& InItem);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	bool IsPressed() const;


	TSharedPtr<const FPlaceableItem> Item;

private:
	
	/** Delegate to execute to get the menu content of this button */
	FOnGetContent OnGetMenuContent;
	
	const FSlateBrush* GetBorder() const;

	bool bIsPressed;

	/** Brush resource that represents a button */
	const FSlateBrush* NormalImage;
	/** Brush resource that represents a button when it is hovered */
	const FSlateBrush* HoverImage;
	/** Brush resource that represents a button when it is pressed */
	const FSlateBrush* PressedImage;
};

class SPlacementAssetMenuEntry : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPlacementAssetMenuEntry){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedPtr<const FPlaceableItem>& InItem);

	virtual FReply OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnDragDetected(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	bool IsPressed() const;

	TSharedPtr<const FPlaceableItem> Item;

	virtual FSlateColor GetForegroundColor() const override;
private:
	bool TryPlacingItemInLevel(TNotNull<ULevel*> DesiredLevel, AActor*& OutPlacedActor);

private:
	const FSlateBrush* GetBorder() const;
	const FSlateBrush* GetIcon() const;

	bool bIsPressed;

	const FButtonStyle* Style;
	
	mutable const FSlateBrush* AssetImage;
};


class SPlacementModeTools : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SPlacementModeTools ){}
	SLATE_END_ARGS();

	void Construct(const FArguments& InArgs, TSharedRef<SDockTab> ParentTab);

	virtual ~SPlacementModeTools();

	/**
	 *Creates and returns the TSharedRef<SPlacementAssetEntry>for the given TSharedPtr<SPlacementAssetEntry>
	 *
	 * @param InItem the FPlaceableItem for which  the TSharedRef<SPlacementAssetEntry> will be created and returned
	 */
	TSharedRef<SWidget> GetPlacementAssetWidget(const TSharedPtr<FPlaceableItem>& InItem) const;

	/**
	 * Given FName CategoryName as the Category that has been clicked, update the FCategoryDrivenContentBuilder with the proper content for the category
	 *
	 * @param CategoryName the name of the currently chosen Category
	 * @param CategoryLabel the label for the currently chosen Category 
	 */
	void UpdateContentForCategory( FName CategoryName, FText CategoryLabel );

private:

	// Begin SWidget
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;
	// End SWidget

private:
	/** Get the identifier of the currently active tab */
	FName GetActiveTab() const;

	/** Changes the active tab */
	void SetActiveTab(FName TabName);

	/** Check if a search is active */
	bool IsSearchActive() const;

	/**
	 * returns true is the favorites category is selected, else it returns false
	 */
	bool IsFavoritesCategorySelected() const;
	
	/** Update the list of shown items */
	void UpdateShownItems();

	/** Requests an update of the shown items */
	void RequestUpdateShownItems();

	/** Requests a refresh of the list of recently placed assets */
	void RequestRefreshRecentlyPlaced( const TArray< FActorPlacementInfo >& RecentlyPlaced );

	/** Requests a refresh of placeable assets in the All Classes category */
	void RequestRefreshAllClasses();

	/** Called when the placement module performs a category refresh */
	void OnCategoryRefresh(FName CategoryName);

	/** Refreshes the list of placement categories */
	void UpdatePlacementCategories();

	/** When the tab is clicked we adjust the check state, so that the right style is displayed. */
	void OnCategoryChanged(const ECheckBoxState NewState, FName InCategory);

	/** Called when the placement mode tools are opened from a tab drawer */
	void OnTabDrawerOpened();

	/** Gets the tab 'active' state, so that we can show the active style */
	ECheckBoxState GetPlacementTabCheckedState( FName CategoryName ) const;

private:

	/** Called when the search text changes */
	void OnSearchChanged(const FText& InFilterText);
	void OnSearchCommitted(const FText& InFilterText, ETextCommit::Type InCommitType);

	/** Get the text that should be highlighted on any items */
	FText GetHighlightText() const;

private:
	/** Flags to refresh particular categories on tick */
	bool bRefreshRecentlyPlaced;
	bool bRefreshAllClasses;

	/** Flag to indicate that we need to update the list of shown items */
	bool bUpdateShownItems;

	/** Flag to indicate that we're programmatically changing the search text and that we should skip updating the placement entries */
	bool bIsRawSearchChange;

	// The text filter used to filter the classes
	typedef TTextFilter<const FPlaceableItem&> FPlacementAssetEntryTextFilter;
	TSharedPtr<FPlacementAssetEntryTextFilter> SearchTextFilter;

	/* The search box used to update the filter text */
	TSharedPtr<SSearchBox> SearchBoxPtr;

	/** Array of filtered items to show in the list view */
	TArray<TSharedPtr<FPlaceableItem>> FilteredItems;

	/** The name of the currently active tab (where no search is active) */
	FName ActiveTabName;

	/** List view that shows placeable items */
	TSharedPtr<SListView<TSharedPtr<FPlaceableItem>>> ListView;

	/** The FCategoryDrivenContentBuilder which will build the UI for this mode, if defined */
	TSharedPtr<FCategoryDrivenContentBuilder> CategoryContentBuilder;

	/** Array of favorite items to show in the favorites view */
	TArray<TSharedPtr<FPlaceableItem>> FavoriteItems;
};
