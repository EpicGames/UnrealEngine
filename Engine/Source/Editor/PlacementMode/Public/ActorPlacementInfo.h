// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

#include "Templates/TypeHash.h"

class FActorPlacementInfo 
{
public:

	FActorPlacementInfo( const FString& InObjectPath, const FString& InFactory )
		: ObjectPath( InObjectPath )
		, Factory( InFactory )
	{
	}

	FActorPlacementInfo( const FString& String )
	{
		if ( !String.Split( FString( TEXT(";") ), &ObjectPath, &Factory ) )
		{
			ObjectPath = String;
		}
	}

	bool operator==( const FActorPlacementInfo& Other ) const
	{
		return ObjectPath == Other.ObjectPath;
	}

	bool operator!=( const FActorPlacementInfo& Other ) const
	{
		return ObjectPath != Other.ObjectPath;
	}

	FString ToString() const
	{
		return ObjectPath + TEXT(";") + Factory;
	}


public:

	FString ObjectPath;
	FString Factory;
};

inline uint32 GetTypeHash(const FActorPlacementInfo& Object)
{
	return HashCombine(GetTypeHash(Object.ObjectPath), GetTypeHash(Object.Factory));
}