// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Misc/Guid.h"
#include "UObject/Class.h"
#include "GameFramework/Actor.h"
#include "AssetRegistry/AssetData.h"
#include "Widgets/SWidget.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "ActorFactories/ActorFactory.h"
#include "ActorPlacementInfo.h"
#include "GameFramework/Volume.h"
#include "Editor.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Textures/SlateIcon.h"

class FNamePermissionList;
struct FPlaceableItem;

/**
 * Struct that defines an identifier for a particular placeable item in this module.
 * Only be obtainable through IPlacementModeModule::RegisterPlaceableItem
 */
class FPlacementModeID
{
public:
	FPlacementModeID(const FPlacementModeID&) = default;
	FPlacementModeID& operator=(const FPlacementModeID&) = default;

private:
	friend class FPlacementModeModule;

	FPlacementModeID() {}

	/** The category this item is held within */
	FName Category;

	/** Unique identifier (always universally unique across categories) */
	FGuid UniqueID;
};

/**
 * Struct providing information for a user category of placement objects
 */
struct FPlacementCategoryInfo
{
	FPlacementCategoryInfo(FText InDisplayName, FSlateIcon InDisplayIcon, FName InHandle, FString InTag, int32 InSortOrder = 0, bool bInSortable = true)
		: DisplayName(InDisplayName), DisplayIcon(InDisplayIcon), UniqueHandle(InHandle), SortOrder(InSortOrder), TagMetaData(MoveTemp(InTag)), bSortable(bInSortable)
	{
	}

	FPlacementCategoryInfo(FText InDisplayName, FName InHandle, FString InTag, int32 InSortOrder = 0, bool bInSortable = true)
		: DisplayName(InDisplayName), DisplayIcon(), UniqueHandle(InHandle), SortOrder(InSortOrder), TagMetaData(MoveTemp(InTag)), bSortable(bInSortable)
	{
	}


	/** This category's display name */
	FText DisplayName;

	/** This category's short display name (optional) */
	FText ShortDisplayName;

	/** This category's representative icon */
	FSlateIcon DisplayIcon;

	/** A unique name for this category */
	FName UniqueHandle;

	/** Sort order for the category tab (lowest first) */
	int32 SortOrder;

	/** Optional tag meta data for the tab widget */
	FString TagMetaData;

	/** Optional generator function used to construct this category's tab content. Called when the tab is activated. */
	TFunction<TSharedRef<SWidget>()> CustomGenerator;

	/** Whether the items in this category are automatically sortable by name. False if the items are already sorted. */
	bool bSortable;

	/**
	 * FPlaceableItems with custom drag handling, populated only if the category handles making
	 * its own draggable items */
	TArray< TSharedRef<FPlaceableItem>>  CustomDraggableItems;
};

/**
 * Structure defining a placeable item in the placement mode panel
 */
struct FPlaceableItem
{
	/**
	* An object which provides handling for Drags.
	*/
	struct FDragHandler : public TSharedFromThis<FDragHandler>
	{
	public:
		DECLARE_DELEGATE_RetVal(TSharedRef<FDragDropOperation>, FGetContentToDrag  );

		/** a delegate thart returns the FDragDropOperation for the  the draggable */
		FGetContentToDrag GetContentToDrag;

		/** the tooltip for the draggable */
		TSharedPtr<IToolTip> ToolTip;
		
		/** the const FSlateBrush* that provides the icon for the draggable */
		const FSlateBrush* IconBrush;
	};
	
	/** Default constructor */
	FPlaceableItem()
		: Factory(nullptr)
	{}

	/** Constructor that takes a specific factory and asset */
	FPlaceableItem(TScriptInterface<IAssetFactoryInterface> InAssetFactory,
		const FAssetData& InAssetData, TOptional<int32> InSortOrder = TOptional<int32>())
		: Factory(Cast<UActorFactory>(InAssetFactory.GetObject()))
		, AssetFactory(InAssetFactory)
		, AssetData(InAssetData)
		, bAlwaysUseGenericThumbnail(false)
		, SortOrder(InSortOrder)
	{
		AutoSetNativeAndDisplayName();
	}

	//~ Note: for the constructors that expect UActorFactory, we can't forward to the IAssetFactoryInterface
	//~ constructors because those currently have a Cast<>() call that requires linking to the module that
	//~ contains IAssetFactoryInterface, which users sometimes don't do. This problem might go away once we
	//~ finished deprecating and removing the Factory member.

	/** Constructor that takes a specific actor factory and asset */
	FPlaceableItem(UActorFactory* InFactory, const FAssetData& InAssetData, 
		TOptional<int32> InSortOrder = TOptional<int32>()) 
		: Factory(InFactory)
		, AssetFactory(InFactory)
		, AssetData(InAssetData)
		, bAlwaysUseGenericThumbnail(false)
		, SortOrder(InSortOrder)
	{
		AutoSetNativeAndDisplayName();
	}

	//~ TODO: Might want a version of this that is works for non-actor factories.
	/** Constructor for any actor factory class */
	 FPlaceableItem(UClass& InActorFactoryClass, TOptional<int32> InSortOrder = TOptional<int32>())
		: Factory(GEditor->FindActorFactoryByClass(&InActorFactoryClass))
		, AssetFactory(Factory)
		, AssetData(Factory ? Factory->GetDefaultActorClass(FAssetData()) : FAssetData())
		, bAlwaysUseGenericThumbnail(false)
		, SortOrder(InSortOrder)
	{
		AutoSetNativeAndDisplayName();
	}

	/** Constructor that takes a specific factory and asset with brush and display name overrides */
	FPlaceableItem(
		 TScriptInterface<IAssetFactoryInterface> InAssetFactory,
		 const FAssetData& InAssetData,
		 FName InClassThumbnailBrushOverride,
		 FName InClassIconBrushOverride,
		 TOptional<FLinearColor> InAssetTypeColorOverride = TOptional<FLinearColor>(),
		 TOptional<int32> InSortOrder = TOptional<int32>(),
		 TOptional<FText> InDisplayName = TOptional<FText>())
		: Factory(Cast<UActorFactory>(InAssetFactory.GetObject()))
		, AssetFactory(InAssetFactory)
		, AssetData(InAssetData)
		, ClassThumbnailBrushOverride(InClassThumbnailBrushOverride)
		, ClassIconBrushOverride(InClassIconBrushOverride)
		, bAlwaysUseGenericThumbnail(true)
		, AssetTypeColorOverride(InAssetTypeColorOverride)
		, SortOrder(InSortOrder)
	{
		AutoSetNativeAndDisplayName();
		if (InDisplayName.IsSet())
		{
			DisplayName = InDisplayName.GetValue();
		}
	}

	FPlaceableItem(
		TSharedPtr<FPlaceableItem::FDragHandler> InDragHandler
		, TOptional<int32> InSortOrder
		, FText InLabel
		, FString InName ):
		NativeName( InName )
		, DisplayName( InLabel )
		, bAlwaysUseGenericThumbnail(false)
		, SortOrder(InSortOrder)
		, DragHandler(InDragHandler)
	{
	}

	/** Constructor for any placeable actor class with associated asset data, brush and display name overrides */
	 FPlaceableItem(
		UClass& InActorFactoryClass,
		const FAssetData& InAssetData,
		FName InClassThumbnailBrushOverride = NAME_None,
		FName InClassIconBrushOverride = NAME_None,
		TOptional<FLinearColor> InAssetTypeColorOverride = TOptional<FLinearColor>(),
		TOptional<int32> InSortOrder = TOptional<int32>(),
		TOptional<FText> InDisplayName = TOptional<FText>()
	 ) 
		 : Factory(GEditor->FindActorFactoryByClass(&InActorFactoryClass))
		 , AssetFactory(Factory)
		 , AssetData(InAssetData)
		 , ClassThumbnailBrushOverride(InClassThumbnailBrushOverride)
		 , ClassIconBrushOverride(InClassIconBrushOverride)
		 , bAlwaysUseGenericThumbnail(true)
		 , AssetTypeColorOverride(InAssetTypeColorOverride)
		 , SortOrder(InSortOrder)
	{
		 AutoSetNativeAndDisplayName();
		 if (InDisplayName.IsSet())
		 {
			 DisplayName = InDisplayName.GetValue();
		 }
	}

	/** Automatically set this item's native and display names from its class or asset */
	void AutoSetNativeAndDisplayName()
	{
		UClass* Class = AssetData.GetClass() == UClass::StaticClass() ? Cast<UClass>(AssetData.GetAsset()) : nullptr;

		if (Class)
		{
			Class->GetName(NativeName);
			DisplayName = Class->GetDisplayNameText();
		}
		else
		{
			NativeName = AssetData.AssetName.ToString();
			if (AssetData.AssetClassPath == UBlueprintGeneratedClass::StaticClass()->GetClassPathName())
			{
				// For BPGC, we do not want to display the "_C" suffix, as the name of the loaded class will not have it.
				NativeName.RemoveFromEnd(TEXT("_C"));
			}
			DisplayName = FText::FromString(NativeName);
		}
	}

	UE_DEPRECATED(4.27, "Use AutoSetNativeAndDisplayName instead")
	void AutoSetDisplayName() { AutoSetNativeAndDisplayName(); }

	/** Return NativeName as an FName (and cache it) */
	FName GetNativeFName() const
	{ 
		if (NativeFName.IsNone() && !NativeName.IsEmpty())
		{
			NativeFName = FName(*NativeName);
		}
		return NativeFName;
	}

public:

	//~TODO: UE_DEPRECATED(5.4, "Use AssetFactory instead.")
	UActorFactory* Factory = nullptr;

	/** The factory used to create an instance of this placeable item */
	TScriptInterface<IAssetFactoryInterface> AssetFactory = nullptr;

	/** Asset data pertaining to the class */
	FAssetData AssetData;
	
	/** This item's native name */
	FString NativeName;

	/** This item's display name */
	FText DisplayName;

	/** Optional override for the thumbnail brush (passed to FClassIconFinder::FindThumbnailForClass in the form ClassThumbnail.<override>) */
	FName ClassThumbnailBrushOverride;

	/** Optional override for the small icon brush */
	FName ClassIconBrushOverride;

	/** Whether to always use the generic thumbnail for this item or not */
	bool bAlwaysUseGenericThumbnail;

	/** Optional overridden color tint for the asset */
	TOptional<FLinearColor> AssetTypeColorOverride;

	/** Optional sort order (lowest first). Overrides default class name sorting. */
	TOptional<int32> SortOrder;

	/** If provided, handles the drag for the item. This should be provided for objects which are not actors or assets, which have built in handling */
	TSharedPtr<FDragHandler> DragHandler;

private:

	/** This item's native name as an FName (initialized on access only) */
	mutable FName NativeFName;
};

/** Structure of built-in placement categories. Defined as functions to enable external use without linkage */
struct FBuiltInPlacementCategories
{
	static FName Favorites()		{ static FName Name("Favorites");		return Name; }
	static FName RecentlyPlaced()	{ static FName Name("RecentlyPlaced");	return Name; }
	static FName Basic()			{ static FName Name("Basic");			return Name; }
	static FName Lights()			{ static FName Name("Lights");			return Name; }
	static FName Shapes()			{ static FName Name("Shapes");			return Name; }
	static FName Visual()			{ static FName Name("Visual");			return Name; }
	static FName Volumes()			{ static FName Name("Volumes");			return Name; }
	static FName AllClasses()		{ static FName Name("AllClasses");		return Name; }
};

class IPlacementModeModule : public IModuleInterface
{
public:

	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPlacementModeModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IPlacementModeModule >( "PlacementMode" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "PlacementMode" );
	}

	/**
	 * Add the specified assets to the recently placed items list
	 */
	virtual void AddToRecentlyPlaced( const TArray< UObject* >& Assets, UActorFactory* FactoryUsed = NULL ) = 0;
	virtual void AddToRecentlyPlaced(const TArray< UObject* >& Assets, TScriptInterface<IAssetFactoryInterface> FactoryUsed)
	{
		// This overload was added later, so we route to old overload, though there shouldn't be any implementers
		// aside from the module itself.
		AddToRecentlyPlaced(Assets, Cast<UActorFactory>(FactoryUsed.GetObject()));
	}
	
	/**
	 * Add the specified asset to the recently placed items list
	 */
	virtual void AddToRecentlyPlaced( UObject* Asset, UActorFactory* FactoryUsed = NULL ) = 0;
	virtual void AddToRecentlyPlaced(UObject* Asset, TScriptInterface<IAssetFactoryInterface> FactoryUsed)
	{
		// This overload was added later, so we route to old overload, though there shouldn't be any implementers
		// aside from the module itself.
		AddToRecentlyPlaced(Asset, Cast<UActorFactory>(FactoryUsed.GetObject()));
	}

	/**
	 * Get a copy of the recently placed items
	 */
	virtual const TArray< FActorPlacementInfo >& GetRecentlyPlaced() const = 0;

	/**
	 * @return the event that is broadcast whenever the user facing list of placement mode categories gets modified
	 */
	DECLARE_EVENT(IPlacementModeModule, FOnPlacementModeCategoryListChanged);
	virtual FOnPlacementModeCategoryListChanged& OnPlacementModeCategoryListChanged() = 0;

	/**
	 * @return the event that is broadcast whenever a placement mode category is refreshed
	 */
	DECLARE_EVENT_OneParam( IPlacementModeModule, FOnPlacementModeCategoryRefreshed, FName /*CategoryName*/ );
	virtual FOnPlacementModeCategoryRefreshed& OnPlacementModeCategoryRefreshed() = 0;

	/**
	 * @return the event that is broadcast whenever the list of recently placed assets changes
	 */
	DECLARE_EVENT_OneParam( IPlacementModeModule, FOnRecentlyPlacedChanged, const TArray< FActorPlacementInfo >& /*NewRecentlyPlaced*/ );
	virtual FOnRecentlyPlacedChanged& OnRecentlyPlacedChanged() = 0;

	/**
	 * @return the event that is broadcast whenever the list of all placeable assets changes
	 */
	DECLARE_EVENT( IPlacementModeModule, FOnAllPlaceableAssetsChanged );
	virtual FOnAllPlaceableAssetsChanged& OnAllPlaceableAssetsChanged() = 0;

	/**
	 * @return the event that is broadcast whenever the filtering of placeable items changes (system filtering, not user filtering)
	 */
	DECLARE_EVENT(IPlacementModeModule, FOnPlaceableItemFilteringChanged);
	virtual FOnPlaceableItemFilteringChanged& OnPlaceableItemFilteringChanged() = 0;

	/**
	 * Creates the placement browser widget
	 */
	virtual TSharedRef<SWidget> CreatePlacementModeBrowser(TSharedRef<SDockTab> ParentTab) = 0;

public:

	/**
	 * Register a new category of placement items
	 *
	 * @param Info		Information pertaining to the category
	 * @return true on success, false on failure (probably if the category's unique handle is already in use)
	 */
	virtual bool RegisterPlacementCategory(const FPlacementCategoryInfo& Info) = 0;

	/**
	 * Unregister a previously registered category
	 *
	 * @param UniqueHandle	The unique handle of the category to unregister
	 */
	virtual void UnregisterPlacementCategory(FName Handle) = 0;

	/**
	 * Retrieve an already registered category
	 *
	 * @param UniqueHandle	The unique handle of the category to retrieve
	 * @return Ptr to the category's information structure, or nullptr if it does not exit
	 */
	virtual const FPlacementCategoryInfo* GetRegisteredPlacementCategory(FName UniqueHandle) const = 0;

	/** Placement categories deny list */
	virtual TSharedRef<FNamePermissionList>& GetCategoryPermissionList() = 0;

	/**
	 * Get all placement categories that aren't denied, sorted by SortOrder
	 *
	 * @param OutCategories	The array to populate with registered category information
	 */
	virtual void GetSortedCategories(TArray<FPlacementCategoryInfo>& OutCategories) const = 0;

	/**
	 * Register a new placeable item for the specified category
	 *
	 * @param Item			The placeable item to register
	 * @param CategoryName	Unique handle to the category in which to place this item
	 * @return Optional unique identifier for the registered item, or empty on failure (if the category doesn't exist)
	 */
	virtual TOptional<FPlacementModeID> RegisterPlaceableItem(FName CategoryName, const TSharedRef<FPlaceableItem>& Item) = 0;

	/**
	 * Unregister a previously registered placeable item
	 *
	 * @param ID			Unique identifier for the item. Will have been obtained from a previous call to RegisterPlaceableItem
	 */
	virtual void UnregisterPlaceableItem(FPlacementModeID ID) = 0;

	typedef TFunction<bool(const TSharedPtr<FPlaceableItem>&)> TPlaceableItemPredicate;

	/** 
	 * Registers system-level (not user) filtering for placeable items. 
	 * An item is displayed if at least one of the predicate returns true or if there's none registered.
	 * @param Predicate Function that returns true if the passed item should be available
	 * @param OwnerName Name of the predicate owner
	 * @return False on failure to register the predicate because one already exists under the specified owner name
	 */
	virtual bool RegisterPlaceableItemFilter(TPlaceableItemPredicate Predicate, FName OwnerName) = 0;

	/**
	 * Registers system-level (not user) filtering for placeable items.
	 * An item is displayed if at least one of the predicate returns true or if there's none registered.
	 */
	virtual void UnregisterPlaceableItemFilter(FName OwnerName) = 0;

	/**
	 * Get all items in a given category, system filtered, unsorted
	 *
	 * @param Category		The unique handle of the category to get items for
	 * @param OutItems		Array to populate with the items in this category
	 */
	virtual void GetItemsForCategory(FName Category, TArray<TSharedPtr<FPlaceableItem>>& OutItems) const = 0;

	/**
	 * Get all items in a given category, system and user filtered, unsorted
	 *
	 * @param Category		The unique handle of the category to get items for
	 * @param OutItems		Array to populate with the items in this category
	 * @param Filter 		Filter predicate used to filter out items. Return true to pass the filter, false otherwise
	 */
	virtual void GetFilteredItemsForCategory(FName Category, TArray<TSharedPtr<FPlaceableItem>>& OutItems, TFunctionRef<bool(const TSharedPtr<FPlaceableItem>&)> Filter) const = 0;

	/**
	 * Puts the items that should be shown for Category CategoryName in the OutItems array
	 * 
	 * @param CategoryName the name of the category to get the items for
	 * @param OutItems the array of FPlaceableItems to put the items in
	 * @param ItemNames the names of the FPlaceableItems to get and put in the OutItems array
	 */
	virtual void GetItemsWithNamesForCategory( FName CategoryName, TArray<TSharedPtr<FPlaceableItem>>& OutItems, const TArray<FName>& ItemNames ) const = 0;

	/**
	 * Instruct the category associated with the specified unique handle that it should regenerate its items
	 *
	 * @param Category		The unique handle of the category to get items for
	 */
	virtual void RegenerateItemsForCategory(FName Category) = 0;
};
