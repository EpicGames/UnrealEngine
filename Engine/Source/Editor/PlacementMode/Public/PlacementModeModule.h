// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPlacementModeModule.h"
#include "Framework/MultiBox/MultiBoxExtender.h"

struct FPlacementCategory : FPlacementCategoryInfo
{
	FPlacementCategory(const FPlacementCategoryInfo& SourceInfo)
		: FPlacementCategoryInfo(SourceInfo)
	{
		
	}

	FPlacementCategory(FPlacementCategory&& In)
		: FPlacementCategoryInfo(MoveTemp(In))
		, Items(MoveTemp(In.Items))
	{}

	FPlacementCategory& operator=(FPlacementCategory&& In)
	{
		FPlacementCategoryInfo::operator=(MoveTemp(In));
		Items = MoveTemp(In.Items);
		return *this;
	}

	TMap<FGuid, TSharedPtr<FPlaceableItem>> Items;
};

static TOptional<FLinearColor> GetBasicShapeColorOverride();

class FPlacementModeModule : public IPlacementModeModule
{
public:

	FPlacementModeModule();

	/**
	 * Called right after the module's DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void PreUnloadCallback() override;

	DECLARE_DERIVED_EVENT(FPlacementModeModule, IPlacementModeModule::FOnPlacementModeCategoryListChanged, FOnPlacementModeCategoryListChanged);
	virtual FOnPlacementModeCategoryListChanged& OnPlacementModeCategoryListChanged() override { return PlacementModeCategoryListChanged; }

	DECLARE_DERIVED_EVENT(FPlacementModeModule, IPlacementModeModule::FOnPlacementModeCategoryRefreshed, FOnPlacementModeCategoryRefreshed);
	virtual FOnPlacementModeCategoryRefreshed& OnPlacementModeCategoryRefreshed() override { return PlacementModeCategoryRefreshed; }

	DECLARE_DERIVED_EVENT(FPlacementModeModule, IPlacementModeModule::FOnRecentlyPlacedChanged, FOnRecentlyPlacedChanged);
	virtual FOnRecentlyPlacedChanged& OnRecentlyPlacedChanged() override { return RecentlyPlacedChanged; }

	DECLARE_DERIVED_EVENT(FPlacementModeModule, IPlacementModeModule::FOnAllPlaceableAssetsChanged, FOnAllPlaceableAssetsChanged);
	virtual FOnAllPlaceableAssetsChanged& OnAllPlaceableAssetsChanged() override { return AllPlaceableAssetsChanged; }

	DECLARE_DERIVED_EVENT(FPlacementModeModule, IPlacementModeModule::FOnPlaceableItemFilteringChanged, FOnPlaceableItemFilteringChanged);
	virtual FOnPlaceableItemFilteringChanged& OnPlaceableItemFilteringChanged() override { return PlaceableItemFilteringChanged; };

	/**
	 * Add the specified assets to the recently placed items list
	 */
	virtual void AddToRecentlyPlaced(const TArray< UObject* >& PlacedObjects, UActorFactory* FactoryUsed = NULL) override;
	virtual void AddToRecentlyPlaced(const TArray< UObject* >& Assets, TScriptInterface<IAssetFactoryInterface> FactoryUsed) override;

	/**
	 * Add the specified asset to the recently placed items list
	 */
	virtual void AddToRecentlyPlaced(UObject* Asset, UActorFactory* FactoryUsed = NULL) override;
	virtual void AddToRecentlyPlaced(UObject* Asset, TScriptInterface<IAssetFactoryInterface> FactoryUsed) override;

	/**
	 * Get a copy of the recently placed items list
	 */
	virtual const TArray< FActorPlacementInfo >& GetRecentlyPlaced() const override
	{
		return RecentlyPlaced;
	}

	virtual TSharedRef<SWidget> CreatePlacementModeBrowser(TSharedRef<SDockTab> ParentTab) override;

	virtual bool RegisterPlacementCategory(const FPlacementCategoryInfo& Info);

	virtual const FPlacementCategoryInfo* GetRegisteredPlacementCategory(FName CategoryName) const override
	{
		return Categories.Find(CategoryName);
	}

	virtual void UnregisterPlacementCategory(FName Handle);

	virtual TSharedRef<FNamePermissionList>& GetCategoryPermissionList() override { return CategoryPermissionList; }

	virtual void GetSortedCategories(TArray<FPlacementCategoryInfo>& OutCategories) const;

	virtual TOptional<FPlacementModeID> RegisterPlaceableItem(FName CategoryName, const TSharedRef<FPlaceableItem>& InItem);

	virtual void UnregisterPlaceableItem(FPlacementModeID ID);

	virtual bool RegisterPlaceableItemFilter(TPlaceableItemPredicate Predicate, FName OwnerName) override;

	virtual void UnregisterPlaceableItemFilter(FName OwnerName) override;

	virtual void GetItemsForCategory(FName CategoryName, TArray<TSharedPtr<FPlaceableItem>>& OutItems) const;

	virtual void GetFilteredItemsForCategory(FName CategoryName, TArray<TSharedPtr<FPlaceableItem>>& OutItems, TFunctionRef<bool(const TSharedPtr<FPlaceableItem>&)> Filter) const;

	/**
	 * Puts the FPlaceableItem instances with names matching the FNames in ItemNames for the placement category CategoryName in the OutItems TArray
	 * 
	 * @param CategoryName the name of the Placement category that contains the FPlaceableItems
	 * @param OutItems  the TArray to fill with the Items that match the names in ItemNames
	 * @param ItemNames the names of the FPlaceableItem instances to put in OutItems
	 */
	virtual void GetItemsWithNamesForCategory( FName CategoryName, TArray<TSharedPtr<FPlaceableItem>>& OutItems, const TArray<FName>& ItemNames  ) const override;

	virtual void RegenerateItemsForCategory(FName Category) override;

private:

	void OnAssetRemoved(const FAssetData& /*InRemovedAssetData*/);
	void OnAssetRenamed(const FAssetData& AssetData, const FString& OldObjectPath);
	void OnAssetAdded(const FAssetData& AssetData);
	void OnInitialAssetsScanComplete();

	void RefreshRecentlyPlaced();
	void RefreshVolumes();
	void RefreshAllPlaceableClasses();

	FGuid CreateID();
	FPlacementModeID CreateID(FName InCategory);

	bool PassesFilters(const TSharedPtr<FPlaceableItem>& Item) const;

	void OnCategoryPermissionListChanged();

private:

	TMap<FName, FPlacementCategory> Categories;

	TSharedRef<FNamePermissionList> CategoryPermissionList;

	TMap<FName, TPlaceableItemPredicate> PlaceableItemPredicates;

	TArray< FActorPlacementInfo > RecentlyPlaced;
	FOnRecentlyPlacedChanged RecentlyPlacedChanged;

	FOnAllPlaceableAssetsChanged AllPlaceableAssetsChanged;
	FOnPlacementModeCategoryRefreshed PlacementModeCategoryRefreshed;
	FOnPlaceableItemFilteringChanged PlaceableItemFilteringChanged;
	FOnPlacementModeCategoryListChanged PlacementModeCategoryListChanged;

	TArray< TSharedPtr<FExtender> > ContentPaletteFiltersExtenders;
	TArray< TSharedPtr<FExtender> > PaletteExtenders;

	// When users explicitly add placeable items, they may add custom icons/descriptions, so we 
	// need to store extra data to be able to recreate the placeable item in "recently placed"
	TMap<FActorPlacementInfo, TWeakPtr<FPlaceableItem>> ManuallyCreatedPlaceableItems;
};
